{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "ChaosVehiclesPlugin",
	"Description": "Chaos Vehicle Integration",
	"Category": "Physics",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "ChaosVehicles",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "ChaosVehiclesEditor",
			"Type": "UncookedOnly",
			"LoadingPhase": "PreDefault"
		}
	]
}