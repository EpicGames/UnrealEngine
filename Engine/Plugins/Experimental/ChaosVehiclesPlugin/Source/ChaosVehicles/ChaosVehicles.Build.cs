// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ChaosVehicles : ModuleRules
	{
        public ChaosVehicles(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
                    "Engine",
                    "EngineSettings",
                    "RenderCore",
                    "AnimGraphRuntime",
                    "RHI",
                    "ChaosVehiclesCore",
					"ChaosVehiclesEngine"
				}
				);

            SetupModulePhysicsSupport(Target);
			PrivateDefinitions.Add("CHAOS_INCLUDE_LEVEL_1=1");
		}
	}
}
