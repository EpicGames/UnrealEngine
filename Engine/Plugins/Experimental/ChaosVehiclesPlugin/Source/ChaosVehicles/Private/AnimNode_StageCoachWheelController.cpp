// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNode_StageCoachWheelController.h"
#include "Animation/AnimTrace.h"
#include "AnimationRuntime.h"
#include "Animation/AnimStats.h"
#include "WheeledVehiclePawn.h"
#include "ChaosVehicleMovementComponent.h"


FAnimNode_StageCoachWheelController::FAnimNode_StageCoachWheelController()
	: WheelSpokeCount(5)
	, MaxAngularVelocity(256.f)
	, ShutterSpeed(30.f)
	, StageCoachBlend(730.f)
{
	AnimInstanceProxy = nullptr;
}

void FAnimNode_StageCoachWheelController::GatherDebugData(FNodeDebugData& DebugData)
{
	FString DebugLine = DebugData.GetNodeName(this);

	DebugLine += "(";
	AddDebugNodeData(DebugLine);
	DebugLine += ")";

	DebugData.AddDebugItem(DebugLine);

	const TArray<FWheelAnimationData>& WheelAnimData = AnimInstanceProxy->GetWheelAnimData();
	for (const FWheelLookupData& Wheel : Wheels)
	{
		if (Wheel.BoneReference.BoneIndex != INDEX_NONE)
		{
			DebugLine = FString::Printf(TEXT(" [Wheel Index : %d] Bone: %s , Rotation Offset : %s, Location Offset : %s"),
				Wheel.WheelIndex, *Wheel.BoneReference.BoneName.ToString(), *WheelAnimData[Wheel.WheelIndex].RotOffset.ToString(), *WheelAnimData[Wheel.WheelIndex].LocOffset.ToString());
		}
		else
		{
			DebugLine = FString::Printf(TEXT(" [Wheel Index : %d] Bone: %s (invalid bone)"),
				Wheel.WheelIndex, *Wheel.BoneReference.BoneName.ToString());
		}

		DebugData.AddDebugItem(DebugLine);
	}

	ComponentPose.GatherDebugData(DebugData);
}

void FAnimNode_StageCoachWheelController::EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms)
{
	check(OutBoneTransforms.Num() == 0);

	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(StageCoachWheelController, !IsInGameThread());

	const TArray<FWheelAnimationData>& WheelAnimData = AnimInstanceProxy->GetWheelAnimData();

	const FBoneContainer& BoneContainer = Output.Pose.GetPose().GetBoneContainer();
	for(const FWheelLookupData& Wheel : Wheels)
	{
		if (Wheel.BoneReference.IsValidToEvaluate(BoneContainer))
		{ 
			if (Wheel.WheelIndex < WheelAnimData.Num())
			{
				FCompactPoseBoneIndex WheelSimBoneIndex = Wheel.BoneReference.GetCompactPoseIndex(BoneContainer);

				// the way we apply transform is same as FMatrix or FTransform
				// we apply scale first, and rotation, and translation
				// if you'd like to translate first, you'll need two nodes that first node does translate and second nodes to rotate. 
				FTransform NewBoneTM = Output.Pose.GetComponentSpaceTransform(WheelSimBoneIndex);

				FAnimationRuntime::ConvertCSTransformToBoneSpace(Output.AnimInstanceProxy->GetComponentTransform(), Output.Pose, NewBoneTM, WheelSimBoneIndex, BCS_ComponentSpace);

				// Apply rotation offset
				const FQuat BoneQuat(WheelAnimData[Wheel.WheelIndex].RotOffset);
				NewBoneTM.SetRotation(BoneQuat * NewBoneTM.GetRotation());

				// Apply loc offset
				NewBoneTM.AddToTranslation(WheelAnimData[Wheel.WheelIndex].LocOffset);

				// Convert back to Component Space.
				FAnimationRuntime::ConvertBoneSpaceTransformToCS(Output.AnimInstanceProxy->GetComponentTransform(), Output.Pose, NewBoneTM, WheelSimBoneIndex, BCS_ComponentSpace);

				// add back to it
				OutBoneTransforms.Add(FBoneTransform(WheelSimBoneIndex, NewBoneTM));
			}
			else
			{
				UE_LOGF(LogChaos, Error, "Invalid condition Wheel.WheelIndex (%d) >= WheelAnimData.Num (%d)", Wheel.WheelIndex, WheelAnimData.Num());
			}
		}
	}

#if ANIM_TRACE_ENABLED
	for (const FWheelLookupData& Wheel : Wheels)
	{
		if ((Wheel.BoneReference.BoneIndex != INDEX_NONE) && (Wheel.WheelIndex < WheelAnimData.Num()))
		{
			TRACE_ANIM_NODE_VALUE(Output, *FString::Printf(TEXT("Wheel %d Name"), Wheel.WheelIndex), *Wheel.BoneReference.BoneName.ToString());
			TRACE_ANIM_NODE_VALUE(Output, *FString::Printf(TEXT("Wheel %d Rotation Offset"), Wheel.WheelIndex), WheelAnimData[Wheel.WheelIndex].RotOffset);
			TRACE_ANIM_NODE_VALUE(Output, *FString::Printf(TEXT("Wheel %d Location Offset"), Wheel.WheelIndex), WheelAnimData[Wheel.WheelIndex].LocOffset);
		}
		else
		{
			TRACE_ANIM_NODE_VALUE(Output, *FString::Printf(TEXT("Wheel %d Name"), Wheel.WheelIndex), *FString::Printf(TEXT("%s (invalid)"), *Wheel.BoneReference.BoneName.ToString()));
		}
	}
#endif
}

bool FAnimNode_StageCoachWheelController::IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones) 
{
	// if both bones are valid
	for (const FWheelLookupData& Wheel : Wheels)
	{
		// if one of them is valid
		if (Wheel.BoneReference.IsValidToEvaluate(RequiredBones) == true)
		{
			return true;
		}
	}

	return false;
}

void FAnimNode_StageCoachWheelController::InitializeBoneReferences(const FBoneContainer& RequiredBones) 
{
	const TArray<FWheelAnimationData>& WheelAnimData = AnimInstanceProxy->GetWheelAnimData();
	const int32 NumWheels = WheelAnimData.Num();
	Wheels.Empty(NumWheels);

	for (int32 WheelIndex = 0; WheelIndex < NumWheels; ++WheelIndex)
	{
		FWheelLookupData* Wheel = new(Wheels)FWheelLookupData();
		Wheel->WheelIndex = WheelIndex;
		Wheel->BoneReference.BoneName = WheelAnimData[WheelIndex].BoneName;
		Wheel->BoneReference.Initialize(RequiredBones);
	}

	// sort by bone indices
	Wheels.Sort([](const FWheelLookupData& L, const FWheelLookupData& R) { return L.BoneReference.BoneIndex < R.BoneReference.BoneIndex; });
}

void FAnimNode_StageCoachWheelController::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	AnimInstanceProxy = (FVehicleAnimationInstanceProxy*)Context.AnimInstanceProxy;

	if (AnimInstanceProxy)
	{
		AnimInstanceProxy->SetStageCoachEffectParams(WheelSpokeCount, MaxAngularVelocity, ShutterSpeed, StageCoachBlend);
	}
}


