// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVehicleManager.h"
#include "Engine/World.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "UObject/UObjectIterator.h"

#include "PhysicalMaterials/PhysicalMaterial.h"
#include "Physics/PhysicsFiltering.h"
#include "Physics/PhysicsInterfaceCore.h"
#include "ChaosVehicleMovementComponent.h"

#include "PBDRigidsSolver.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"

DECLARE_CYCLE_STAT(TEXT("VehicleManager:ParallelUpdateVehicles"), STAT_ChaosVehicleManager_ParallelUpdateVehicles, STATGROUP_ChaosVehicleManager);
DECLARE_CYCLE_STAT(TEXT("VehicleManager:Update"), STAT_ChaosVehicleManager_Update, STATGROUP_ChaosVehicleManager);
DECLARE_CYCLE_STAT(TEXT("VehicleManager:ScenePreTick"), STAT_ChaosVehicleManager_ScenePreTick, STATGROUP_ChaosVehicleManager);

DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("NumVehiclesTotal"), STAT_NumVehicles_Dynamic, STATGROUP_ChaosVehicleManager);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("NumVehiclesAwake"), STAT_NumVehicles_Awake, STATGROUP_ChaosVehicleManager);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("NumVehiclesSleeping"), STAT_NumVehicles_Sleeping, STATGROUP_ChaosVehicleManager);

extern FVehicleDebugParams GVehicleDebugParams;

TMap<FPhysScene*, FChaosVehicleManager*> FChaosVehicleManager::SceneToVehicleManagerMap;
uint32 FChaosVehicleManager::VehicleSetupTag = 0;

FDelegateHandle FChaosVehicleManager::OnPostWorldInitializationHandle;
FDelegateHandle FChaosVehicleManager::OnWorldCleanupHandle;

bool FChaosVehicleManager::GInitialized = false;

void OnPostWorldInitialize(UWorld* InWorld, const UWorld::InitializationValues)
{
	FChaosVehicleManager* Manager = FChaosVehicleManager::GetVehicleManagerFromScene(InWorld->GetPhysicsScene());
	if(Manager)
	{
		Manager->RegisterCallbacks();
	}
}

void OnWorldCleanup(UWorld* InWorld, bool bSessionEnded, bool bCleanupResources)
{
	FChaosVehicleManager* Manager = FChaosVehicleManager::GetVehicleManagerFromScene(InWorld->GetPhysicsScene());
	if(Manager)
	{
		Manager->UnregisterCallbacks();
	}
}

FChaosVehicleManager::FChaosVehicleManager(FPhysScene* PhysScene)
	: Scene(*PhysScene)
	, AsyncCallback(nullptr)
	, Timestamp(0)
{
	check(PhysScene);

	if (!GInitialized)
	{
		GInitialized = true;
		// PhysScene->GetOwningWorld() is always null here, the world is being setup too late to be of use
		// therefore setup these global world delegates that will callback when everything is setup so registering
		// the physics solver Async Callback will succeed
		OnPostWorldInitializationHandle = FWorldDelegates::OnPostWorldInitialization.AddStatic(&OnPostWorldInitialize);
		OnWorldCleanupHandle = FWorldDelegates::OnWorldCleanup.AddStatic(&OnWorldCleanup);
	}

	ensure(FChaosVehicleManager::SceneToVehicleManagerMap.Find(PhysScene) == nullptr);	//double registration with same scene, will cause a leak

	// Add to Scene-To-Manager map
	FChaosVehicleManager::SceneToVehicleManagerMap.Add(PhysScene, this);
}


void FChaosVehicleManager::RegisterCallbacks()
{
	OnPhysScenePreTickHandle = Scene.OnPhysScenePreTick.AddRaw(this, &FChaosVehicleManager::Update);
	OnPhysScenePostTickHandle = Scene.OnPhysScenePostTick.AddRaw(this, &FChaosVehicleManager::PostUpdate);

	check(AsyncCallback == nullptr);
	AsyncCallback = Scene.GetSolver()->CreateAndRegisterSimCallbackObject_External<FChaosVehicleManagerAsyncCallback>();
}

void FChaosVehicleManager::UnregisterCallbacks()
{
	Scene.OnPhysScenePreTick.Remove(OnPhysScenePreTickHandle);
	Scene.OnPhysScenePostTick.Remove(OnPhysScenePostTickHandle);

	if (AsyncCallback)
	{
		Scene.GetSolver()->UnregisterAndFreeSimCallbackObject_External(AsyncCallback);
		AsyncCallback = nullptr;
	}
}

void FChaosVehicleManager::DetachFromPhysScene(FPhysScene* PhysScene)
{
	if (AsyncCallback)
	{
		UnregisterCallbacks();
	}

	FChaosVehicleManager::SceneToVehicleManagerMap.Remove(PhysScene);
}

FChaosVehicleManager::~FChaosVehicleManager()
{
	while (Vehicles.Num() > 0)
	{
		RemoveVehicle(Vehicles.Last());
	}
}

FChaosVehicleManager* FChaosVehicleManager::GetVehicleManagerFromScene(FPhysScene* PhysScene)
{
	FChaosVehicleManager* Manager = nullptr;
	FChaosVehicleManager** ManagerPtr = SceneToVehicleManagerMap.Find(PhysScene);
	if (ManagerPtr != nullptr)
	{
		Manager = *ManagerPtr;
	}
	return Manager;
}

void FChaosVehicleManager::AddVehicle(TWeakObjectPtr<UChaosVehicleMovementComponent> Vehicle)
{
	check(Vehicle != NULL);
	check(Vehicle->PhysicsVehicleOutput());
	check(AsyncCallback);

	Vehicles.Add(Vehicle);
}

void FChaosVehicleManager::RemoveVehicle(TWeakObjectPtr<UChaosVehicleMovementComponent> Vehicle)
{
	check(Vehicle != NULL);
	check(Vehicle->PhysicsVehicleOutput());

	Vehicles.Remove(Vehicle);

	if (Vehicle->PhysicsVehicleOutput().IsValid())
	{
		Vehicle->PhysicsVehicleOutput().Reset(nullptr);
	}

}

void FChaosVehicleManager::ScenePreTick(FPhysScene* PhysScene, float DeltaTime)
{
	// inputs being set via back door, i.e. accessing PVehicle directly is a no go now, needs to go through async input system
	SCOPE_CYCLE_COUNTER(STAT_ChaosVehicleManager_ScenePreTick);

	for (int32 i = 0; i < Vehicles.Num(); ++i)
	{
		Vehicles[i]->PreTickGT(DeltaTime);
	}

}

void FChaosVehicleManager::Update(FPhysScene* PhysScene, float DeltaTime)
{
	SCOPE_CYCLE_COUNTER(STAT_ChaosVehicleManager_Update);

	UWorld* World = Scene.GetOwningWorld();

	SubStepCount = 0;

	ScenePreTick(PhysScene, DeltaTime);

	ParallelUpdateVehicles(DeltaTime);

	if (World)
	{
		FChaosVehicleManagerAsyncInput* AsyncInput = AsyncCallback->GetProducerInputData_External();
		for (TWeakObjectPtr<UChaosVehicleMovementComponent> Vehicle : Vehicles)
		{
			Vehicle->Update(DeltaTime);
			Vehicle->FinalizeSimCallbackData(*AsyncInput);
		}
	}
}

void FChaosVehicleManager::PostUpdate(FChaosScene* PhysScene)
{
	SET_DWORD_STAT(STAT_NumVehicles_Dynamic, Vehicles.Num());

	int32 SleepingCount = 0;
	for (int32 i = 0; i < Vehicles.Num(); ++i)
	{
		if (Vehicles[i]->VehicleState.bSleeping)
		{
			SleepingCount++;
		}
	}
	SET_DWORD_STAT(STAT_NumVehicles_Awake, Vehicles.Num() - SleepingCount);
	SET_DWORD_STAT(STAT_NumVehicles_Sleeping, SleepingCount);

}

void FChaosVehicleManager::ParallelUpdateVehicles(float DeltaSeconds)
{
	SCOPE_CYCLE_COUNTER(STAT_ChaosVehicleManager_ParallelUpdateVehicles);

	FChaosVehicleManagerAsyncInput* AsyncInput = AsyncCallback->GetProducerInputData_External();

	AsyncInput->Reset();	//only want latest frame's data

	{
		// We pass pointers from TArray so this reserve is critical. Otherwise realloc happens
		AsyncInput->VehicleInputs.Reserve(Vehicles.Num());
		AsyncInput->Timestamp = Timestamp;
		AsyncInput->World = Scene.GetOwningWorld();
	}

	// Grab all outputs for processing, even future ones for interpolation.
	{
		Chaos::TSimCallbackOutputHandle<FChaosVehicleManagerAsyncOutput> AsyncOutputLatest;
		while ((AsyncOutputLatest = AsyncCallback->PopFutureOutputData_External()))
		{
			PendingOutputs.Emplace(MoveTemp(AsyncOutputLatest));
		}
	}

	// Since we are in pre-physics, delta seconds is not accounted for in external time yet
	const double ResultsTime = AsyncCallback->GetSolver()->GetPhysicsResultsTime_External() + DeltaSeconds;

	// Find index of first non-consumable output (first one after current time)
	int32 LastOutputIdx = 0;
	for (; LastOutputIdx < PendingOutputs.Num(); ++LastOutputIdx)
	{
		if (PendingOutputs[LastOutputIdx]->InternalTime > ResultsTime)
		{
			break;
		}
	}

	// Process events on all outputs which occurred before current time
	// 
	//for (int32 OutputIdx = 0; OutputIdx < LastOutputIdx; ++OutputIdx)
	//{
	//	for (TWeakObjectPtr<UChaosVehicleMovementComponent> Vehicle : Vehicles)
	//	{
	//		Vehicle->GameThread_ProcessIntermediateAsyncOutput(*PendingOutputs[OutputIdx]);
	//	}
	//}

	// Cache the last consumed output for interpolation
	if (LastOutputIdx > 0)
	{
		LatestOutput = MoveTemp(PendingOutputs[LastOutputIdx - 1]);
	}

	// Remove all consumed outputs
	{
		TArray<Chaos::TSimCallbackOutputHandle<FChaosVehicleManagerAsyncOutput>> NewPendingOutputs;
		for (int32 OutputIdx = LastOutputIdx; OutputIdx < PendingOutputs.Num(); ++OutputIdx)
		{
			NewPendingOutputs.Emplace(MoveTemp(PendingOutputs[OutputIdx]));
		}
		PendingOutputs = MoveTemp(NewPendingOutputs);
	}

	// It's possible we will end up multiple frames ahead of output, take the latest ready output.
	Chaos::TSimCallbackOutputHandle<FChaosVehicleManagerAsyncOutput> AsyncOutput;
	Chaos::TSimCallbackOutputHandle<FChaosVehicleManagerAsyncOutput> AsyncOutputLatest;
	while ((AsyncOutputLatest = AsyncCallback->PopOutputData_External()))
	{
		AsyncOutput = MoveTemp(AsyncOutputLatest);

		// Note: not used - left as a reminder
		//for (TWeakObjectPtr<UChaosVehicleMovementComponent> Vehicle : Vehicles)
		//{
		//	//Vehicle->GameThread_ProcessIntermediateAsyncOutput(*AsyncOutput);
		//}
	}

	if (UWorld* World = Scene.GetOwningWorld())
	{
		int32 NumVehiclesInActiveBatch = 0;
		for (TWeakObjectPtr<UChaosVehicleMovementComponent> Vehicle : Vehicles)
		{
			auto NextOutput = PendingOutputs.Num() > 0 ? PendingOutputs[0].Get() : nullptr;
			float Alpha = 0.f;
			if (NextOutput && LatestOutput)
			{
				const float Denom = NextOutput->InternalTime - LatestOutput->InternalTime;
				if (Denom > SMALL_NUMBER)
				{
					Alpha = (ResultsTime - LatestOutput->InternalTime) / Denom;
				}
			}

			AsyncInput->VehicleInputs.Add(Vehicle->SetCurrentAsyncInputOutput(AsyncInput->VehicleInputs.Num(), LatestOutput.Get(), NextOutput, Alpha, Timestamp));
		}
	}

	++Timestamp;

	const auto& AwakeVehiclesBatch = Vehicles; // TODO: process awake only

	auto LambdaParallelUpdate = [DeltaSeconds, &AwakeVehiclesBatch](int32 Idx)
	{
		TWeakObjectPtr<UChaosVehicleMovementComponent> Vehicle = AwakeVehiclesBatch[Idx];
		Vehicle->ParallelUpdate(DeltaSeconds); // gets output state from PT
	};

	bool ForceSingleThread = !GVehicleDebugParams.EnableMultithreading;
	ParallelFor(AwakeVehiclesBatch.Num(), LambdaParallelUpdate, ForceSingleThread);
}
