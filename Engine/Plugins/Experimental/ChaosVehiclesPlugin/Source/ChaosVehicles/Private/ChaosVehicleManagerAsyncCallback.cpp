// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVehicleManagerAsyncCallback.h"
#include "ChaosVehicleMovementComponent.h"
#include "PBDRigidsSolver.h"
#include "TransmissionSystem.h"
#include "Chaos/ParticleHandleFwd.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"

extern FVehicleDebugParams GVehicleDebugParams;

DECLARE_CYCLE_STAT(TEXT("AsyncCallback:OnPreSimulate_Internal"), STAT_AsyncCallback_OnPreSimulate, STATGROUP_ChaosVehicleManager);

FName FChaosVehicleManagerAsyncCallback::GetFNameForStatId() const
{
	const static FLazyName StaticName("FChaosVehicleManagerAsyncCallback");
	return StaticName;
}

/**
 * Callback from Physics thread
 */

void FChaosVehicleManagerAsyncCallback::ProcessInputs_Internal(int32 PhysicsStep) 
{
	const FChaosVehicleManagerAsyncInput* AsyncInput = GetConsumerInput_Internal();
	if (AsyncInput == nullptr)
	{
		return;
	}

	for (const TUniquePtr<FChaosVehicleAsyncInput>& VehicleInput : AsyncInput->VehicleInputs)
	{
#if UE_WITH_REMOTE_OBJECT_HANDLE
		if (UChaosVehicleMovementComponent* VehicleComp = VehicleInput->Vehicle)
		{
			VehicleComp->ApplyMigratedVehicleState();
		}
#endif
		UChaosVehicleSimulation* VehicleSim = VehicleInput->Vehicle->VehicleSimulationPT.Get();

		if (VehicleSim == nullptr || !VehicleInput->Vehicle->bUsingNetworkPhysicsPrediction)
		{
			continue;
		}
		bool bIsResimming = false;
		if (UWorld* World = VehicleInput->Vehicle->GetWorld())
		{
			if (FPhysScene* PhysScene = World->GetPhysicsScene())
			{
				if (Chaos::FPhysicsSolver* LocalSolver = PhysScene->GetSolver())
				{
					bIsResimming = LocalSolver->GetEvolution()->IsResimming();
				}
			}
		}

		APlayerController* PlayerController = VehicleInput->Vehicle->GetPlayerController();
		if(PlayerController && PlayerController->IsLocalController() && !bIsResimming)
		{
			VehicleSim->VehicleInputs = VehicleInput->PhysicsInputs.NetworkInputs.VehicleInputs;

			// Apply the GT's gear decision to the PT transmission so it's authoritative
			// for the current frame and gets captured correctly by BuildData
			if (VehicleSim->PVehicle && VehicleSim->PVehicle->HasTransmission())
			{
				Chaos::FSimpleTransmissionSim& Transmission = VehicleSim->PVehicle->GetTransmission();
				Transmission.SetGear(VehicleInput->PhysicsInputs.NetworkInputs.TransmissionTargetGear, true);
			}
		}
		else
		{
			VehicleInput->PhysicsInputs.NetworkInputs.VehicleInputs = VehicleSim->VehicleInputs;
		}
	}
}

void FChaosVehicleManagerAsyncCallback::OnPreSimulate_Internal()
{
	using namespace Chaos;

	SCOPE_CYCLE_COUNTER(STAT_AsyncCallback_OnPreSimulate);

	float DeltaTime = GetDeltaTime_Internal();
	float SimTime = GetSimTime_Internal();

	const FChaosVehicleManagerAsyncInput* Input = GetConsumerInput_Internal();
	if (Input == nullptr)
	{
		return;
	}

	const int32 NumVehicles = Input->VehicleInputs.Num();

	UWorld* World = Input->World.Get();	//only safe to access for scene queries
	if (World == nullptr || NumVehicles == 0)
	{
		//world is gone so don't bother, or nothing to simulate.
		return;
	}

	Chaos::FPhysicsSolver* PhysicsSolver = static_cast<Chaos::FPhysicsSolver*>(GetSolver());
	if (PhysicsSolver == nullptr)
	{
		return;
	}

	FChaosVehicleManagerAsyncOutput& Output = GetProducerOutputData_Internal();
	Output.VehicleOutputs.AddDefaulted(NumVehicles);
	Output.Timestamp = Input->Timestamp;

	const TArray<TUniquePtr<FChaosVehicleAsyncInput>>& InputVehiclesBatch = Input->VehicleInputs;
	TArray<TUniquePtr<FChaosVehicleAsyncOutput>>& OutputVehiclesBatch = Output.VehicleOutputs;

	// beware running the vehicle simulation in parallel, code must remain threadsafe
	auto LambdaParallelUpdate = [World, DeltaTime, SimTime, &InputVehiclesBatch, &OutputVehiclesBatch](int32 Idx)
	{
		const FChaosVehicleAsyncInput& VehicleInput = *InputVehiclesBatch[Idx];

		if (VehicleInput.Proxy == nullptr || VehicleInput.Proxy->GetPhysicsThreadAPI() == nullptr)
		{
			return;
		}

		Chaos::FRigidBodyHandle_Internal* Handle = VehicleInput.Proxy->GetPhysicsThreadAPI();
		if (Handle->ObjectState() != Chaos::EObjectStateType::Dynamic)
		{
			return;
		}

		bool bWake = false;
		OutputVehiclesBatch[Idx] = VehicleInput.Simulate(World, DeltaTime, SimTime, bWake);
	};

	bool ForceSingleThread = !GVehicleDebugParams.EnableMultithreading;
	PhysicsParallelFor(OutputVehiclesBatch.Num(), LambdaParallelUpdate, ForceSingleThread);

	// Delayed application of forces - This is separate from Simulate because forces cannot be executed multi-threaded
	for (const TUniquePtr<FChaosVehicleAsyncInput>& VehicleInput : InputVehiclesBatch)
	{
		if (VehicleInput.IsValid() && VehicleInput->Proxy)
		{
			if (Chaos::FRigidBodyHandle_Internal* Handle = VehicleInput->Proxy->GetPhysicsThreadAPI())
			{
				VehicleInput->ApplyDeferredForces(Handle);
			}
		}
	}
}

TUniquePtr<FChaosVehicleAsyncOutput> FChaosVehicleAsyncInput::Simulate(UWorld* World, const float DeltaSeconds, const float TotalSeconds, bool& bWakeOut) const
{
	TUniquePtr<FChaosVehicleAsyncOutput> Output = MakeUnique<FChaosVehicleAsyncOutput>();

	//UE_LOGF(LogChaos, Warning, "Vehicle Physics Thread Tick %f", DeltaSeconds);

	//support nullptr because it allows us to go wide on filling the async inputs
	if (Proxy == nullptr)
	{
		return Output;
	}

	// We now have access to the physics representation of the chassis on the physics thread async tick
	Chaos::FRigidBodyHandle_Internal* Handle = Proxy->GetPhysicsThreadAPI();

	// FILL OUTPUT DATA HERE THAT WILL GET PASSED BACK TO THE GAME THREAD
	Vehicle->VehicleSimulationPT->TickVehicle(World, DeltaSeconds, *this, *Output.Get(), Handle);

	Output->bValid = true;

	return MoveTemp(Output);
}

void FChaosVehicleAsyncInput::ApplyDeferredForces(Chaos::FRigidBodyHandle_Internal* RigidHandle) const
{
	check(Vehicle);
	check(Vehicle->VehicleSimulationPT);
	Vehicle->VehicleSimulationPT->ApplyDeferredForces(RigidHandle);
}

bool FNetworkVehicleInputs::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	FNetworkPhysicsData::SerializeFrames(Ar);

	Ar << VehicleInputs.SteeringInput;
	Ar << VehicleInputs.ThrottleInput;
	Ar << VehicleInputs.BrakeInput;
	Ar << VehicleInputs.PitchInput;
	Ar << VehicleInputs.RollInput;
	Ar << VehicleInputs.YawInput;
	Ar << VehicleInputs.HandbrakeInput;

	Ar << VehicleInputs.ParkingEnabled;
	Ar << VehicleInputs.GearUpInput;
	Ar << VehicleInputs.GearDownInput;
	Ar << VehicleInputs.TransmissionType;

	Ar << TransmissionChangeTime;
	Ar << TransmissionCurrentGear;
	Ar << TransmissionTargetGear;

	bOutSuccess = true;
	return bOutSuccess;
}

void FNetworkVehicleInputs::ApplyData(UActorComponent* NetworkComponent) const
{
	if (UChaosVehicleSimulation* VehicleSimulation = Cast<UChaosVehicleMovementComponent>(NetworkComponent)->VehicleSimulationPT.Get())
	{
		VehicleSimulation->VehicleInputs = VehicleInputs;

		if (TUniquePtr<Chaos::FSimpleWheeledVehicle>& Vehicle = VehicleSimulation->PVehicle)
		{
			if (Vehicle->HasTransmission())
			{
				Chaos::FSimpleTransmissionSim& Transmission = VehicleSimulation->PVehicle->GetTransmission();
				Transmission.SetCurrentGear(TransmissionCurrentGear);
				Transmission.SetTargetGear(TransmissionTargetGear);
				Transmission.SetCurrentGearChangeTime(TransmissionChangeTime);
			}
		}
	}
}

void FNetworkVehicleInputs::BuildData(const UActorComponent* NetworkComponent)
{
	if (NetworkComponent)
	{
		if (const UChaosVehicleSimulation* VehicleSimulation = Cast<const UChaosVehicleMovementComponent>(NetworkComponent)->VehicleSimulationPT.Get())
		{
			VehicleInputs = VehicleSimulation->VehicleInputs;

			if (const TUniquePtr<Chaos::FSimpleWheeledVehicle>& Vehicle = VehicleSimulation->PVehicle)
			{
				if (Vehicle->HasTransmission())
				{
					Chaos::FSimpleTransmissionSim& Transmission = Vehicle->GetTransmission();
					TransmissionCurrentGear = Transmission.GetCurrentGear();
					TransmissionTargetGear = Transmission.GetTargetGear();
					TransmissionChangeTime = Transmission.GetCurrentGearChangeTime();
				}
			}
		}
	}
}

void FNetworkVehicleInputs::InterpolateData(const FNetworkPhysicsData& MinData, const FNetworkPhysicsData& MaxData)
{
	const FNetworkVehicleInputs& MinInput = static_cast<const FNetworkVehicleInputs&>(MinData);
	const FNetworkVehicleInputs& MaxInput = static_cast<const FNetworkVehicleInputs&>(MaxData);

	const int32 FrameSpan = MaxInput.LocalFrame - MinInput.LocalFrame;
	const float LerpFactor = FrameSpan > 0 ? static_cast<float>(LocalFrame - MinInput.LocalFrame) / static_cast<float>(FrameSpan) : 1.0f;

	TransmissionChangeTime = FMath::Lerp(MinInput.TransmissionChangeTime, MaxInput.TransmissionChangeTime, LerpFactor);
	TransmissionCurrentGear = LerpFactor < 0.5 ? MinInput.TransmissionCurrentGear : MaxInput.TransmissionCurrentGear;
	TransmissionTargetGear = LerpFactor < 0.5 ? MinInput.TransmissionTargetGear : MaxInput.TransmissionTargetGear;

	VehicleInputs.BrakeInput = FMath::Lerp(MinInput.VehicleInputs.BrakeInput, MaxInput.VehicleInputs.BrakeInput, LerpFactor);
	VehicleInputs.HandbrakeInput = FMath::Lerp(MinInput.VehicleInputs.HandbrakeInput, MaxInput.VehicleInputs.HandbrakeInput, LerpFactor);
	VehicleInputs.PitchInput = FMath::Lerp(MinInput.VehicleInputs.PitchInput, MaxInput.VehicleInputs.PitchInput, LerpFactor);
	VehicleInputs.RollInput = FMath::Lerp(MinInput.VehicleInputs.RollInput, MaxInput.VehicleInputs.RollInput, LerpFactor);
	VehicleInputs.ThrottleInput = FMath::Lerp(MinInput.VehicleInputs.ThrottleInput, MaxInput.VehicleInputs.ThrottleInput, LerpFactor);
	VehicleInputs.SteeringInput = FMath::Lerp(MinInput.VehicleInputs.SteeringInput, MaxInput.VehicleInputs.SteeringInput, LerpFactor);
	VehicleInputs.YawInput = FMath::Lerp(MinInput.VehicleInputs.YawInput, MaxInput.VehicleInputs.YawInput, LerpFactor);

	VehicleInputs.ParkingEnabled = LerpFactor < 0.5 ? MinInput.VehicleInputs.ParkingEnabled : MaxInput.VehicleInputs.ParkingEnabled;
	VehicleInputs.GearDownInput = LerpFactor < 0.5 ? MinInput.VehicleInputs.GearDownInput : MaxInput.VehicleInputs.GearDownInput;
	VehicleInputs.GearUpInput = LerpFactor < 0.5 ? MinInput.VehicleInputs.GearUpInput : MaxInput.VehicleInputs.GearUpInput;
	VehicleInputs.TransmissionType = LerpFactor < 0.5 ? MinInput.VehicleInputs.TransmissionType : MaxInput.VehicleInputs.TransmissionType;
}

void FNetworkVehicleInputs::MergeData(const FNetworkPhysicsData& FromData)
{
	const FNetworkVehicleInputs& From = static_cast<const FNetworkVehicleInputs&>(FromData);

	// Floats: average adjacent frames
	TransmissionChangeTime = (TransmissionChangeTime + From.TransmissionChangeTime) * 0.5f;
	VehicleInputs.BrakeInput = (VehicleInputs.BrakeInput + From.VehicleInputs.BrakeInput) * 0.5f;
	VehicleInputs.HandbrakeInput = (VehicleInputs.HandbrakeInput + From.VehicleInputs.HandbrakeInput) * 0.5f;
	VehicleInputs.PitchInput = (VehicleInputs.PitchInput + From.VehicleInputs.PitchInput) * 0.5f;
	VehicleInputs.RollInput = (VehicleInputs.RollInput + From.VehicleInputs.RollInput) * 0.5f;
	VehicleInputs.ThrottleInput = (VehicleInputs.ThrottleInput + From.VehicleInputs.ThrottleInput) * 0.5f;
	VehicleInputs.SteeringInput = (VehicleInputs.SteeringInput + From.VehicleInputs.SteeringInput) * 0.5f;
	VehicleInputs.YawInput = (VehicleInputs.YawInput + From.VehicleInputs.YawInput) * 0.5f;

	// Booleans: additive (true if either frame had it true)
	VehicleInputs.ParkingEnabled |= From.VehicleInputs.ParkingEnabled;

	// Discrete values: TransmissionCurrentGear, TransmissionTargetGear, VehicleInputs.TransmissionType are kept from *this (the newer/accumulated frame) and left unchanged here.
	// Note, also keep GearUpInput and GearDownInput from the latest input as to not apply a previous gear up on top of the latest TransmissionCurrentGear, TransmissionTargetGear
}

void FNetworkVehicleInputs::DecayData(float DecayAmount)
{
	// Vehicle steering inputs are more sensitive to stale predictions; apply 2x the base
	// decay rate to pull them toward neutral faster during input extrapolation.
	DecayAmount = FMath::Min(DecayAmount * 2, 1.0f);

	// Apply decay on steering inputs
	VehicleInputs.PitchInput = FMath::Lerp(VehicleInputs.PitchInput, 0.0f, DecayAmount);
	VehicleInputs.RollInput = FMath::Lerp(VehicleInputs.RollInput, 0.0f, DecayAmount);
	VehicleInputs.SteeringInput = FMath::Lerp(VehicleInputs.SteeringInput, 0.0f, DecayAmount);
	VehicleInputs.YawInput = FMath::Lerp(VehicleInputs.YawInput, 0.0f, DecayAmount);
}

const FString FNetworkVehicleInputs::DebugData()
{
	return FString::Printf(TEXT("FNetworkVehicleInputs | Throttle = %f Brake = %f Roll = %f Pitch = %f Yaw = %f Steering = %f Handbrake = %f Gear = %d | VehicleInputs size = %zd | ControlInputs size = %zd | NetworkInputs = %zd"),
			VehicleInputs.ThrottleInput, VehicleInputs.BrakeInput, VehicleInputs.RollInput, VehicleInputs.PitchInput,
			VehicleInputs.YawInput, VehicleInputs.SteeringInput, VehicleInputs.HandbrakeInput, TransmissionTargetGear, sizeof(FVehicleInputs) * 8, sizeof(FControlInputs) * 8, sizeof(FNetworkVehicleInputs) * 8);
}

bool FNetworkVehicleStates::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	FNetworkPhysicsData::SerializeFrames(Ar);

	Ar << StateLastVelocity;
	Ar << EngineOmega;

	int32 NumWheels = WheelsOmega.Num();
	Ar << NumWheels;

	int32 NumLength = SuspensionAveragedLength.Num();
	Ar << NumLength;

	if (Ar.IsLoading())
	{
		WheelsOmega.SetNum(NumWheels);
		WheelsAngularPosition.SetNum(NumWheels);
		SuspensionLastDisplacement.SetNum(NumWheels);
		SuspensionLastSpringLength.SetNum(NumWheels);
		SuspensionAveragedCount.SetNum(NumWheels);
		SuspensionAveragedNum.SetNum(NumWheels);
		SuspensionAveragedLength.SetNum(NumLength);
	}

	for (int32 WheelIdx = 0; WheelIdx < NumWheels; ++WheelIdx)
	{
		Ar << WheelsOmega[WheelIdx];
		Ar << WheelsAngularPosition[WheelIdx];
		Ar << SuspensionLastDisplacement[WheelIdx];
		Ar << SuspensionLastSpringLength[WheelIdx];
		Ar << SuspensionAveragedCount[WheelIdx];
		Ar << SuspensionAveragedNum[WheelIdx];
	}

	for (int32 LengthIdx = 0; LengthIdx < NumLength; ++LengthIdx)
	{
		Ar << SuspensionAveragedLength[LengthIdx];
	}
	return true;
}

void FNetworkVehicleStates::ApplyData(UActorComponent* NetworkComponent) const
{
	if (UChaosVehicleSimulation* VehicleSimulation = Cast<UChaosVehicleMovementComponent>(NetworkComponent)->VehicleSimulationPT.Get())
	{
		VehicleSimulation->VehicleState.LastFrameVehicleLocalVelocity = StateLastVelocity;
		
		if (TUniquePtr<Chaos::FSimpleWheeledVehicle>& Vehicle = VehicleSimulation->PVehicle)
		{
			if (Vehicle->HasEngine())
			{
				Vehicle->GetEngine().SetEngineOmega(EngineOmega);
			}

			// get the size of the source data - all arrays are always going to be the same length		
			const int32 NumWheels = FMath::Min(SuspensionLastSpringLength.Num(), Vehicle->Wheels.Num());

			int32 LengthCount = 0;
			for (int32 WheelIdx = 0; WheelIdx < NumWheels; ++WheelIdx)
			{
				Chaos::FSimpleSuspensionSim& Suspension = Vehicle->GetSuspension(WheelIdx);
				Suspension.SetLastSpringLength(SuspensionLastSpringLength[WheelIdx]);
				Suspension.SetLastDisplacement(SuspensionLastDisplacement[WheelIdx]);
				Suspension.SetAveragingCount(SuspensionAveragedCount[WheelIdx]);
				Suspension.SetAveragingNum(SuspensionAveragedNum[WheelIdx]);
		
				for (int32 LengthIdx = 0; LengthIdx < Suspension.GetAveragingNum(); ++LengthIdx)
				{
					Suspension.SetAveragingLength(LengthIdx, SuspensionAveragedLength[LengthCount++]);
				}
		
				Chaos::FSimpleWheelSim& Wheel = Vehicle->GetWheel(WheelIdx);
				Wheel.Omega = WheelsOmega[WheelIdx];
				Wheel.AngularPosition = WheelsAngularPosition[WheelIdx];
			}
		}
	}
}

void FNetworkVehicleStates::BuildData(const UActorComponent* NetworkComponent)
{
	if (NetworkComponent)
	{
		if (const UChaosVehicleSimulation* VehicleSimulation = Cast<const UChaosVehicleMovementComponent>(NetworkComponent)->VehicleSimulationPT.Get())
		{
			StateLastVelocity = VehicleSimulation->VehicleState.LastFrameVehicleLocalVelocity;
			if (const TUniquePtr<Chaos::FSimpleWheeledVehicle>& Vehicle = VehicleSimulation->PVehicle)
			{
				if (Vehicle->HasEngine())
				{
					EngineOmega = Vehicle->GetEngine().GetEngineOmega();
				}
				
				const int32 NumWheels = Vehicle->Wheels.Num();
				
				int32 NumLength = 0;
				for (int32 WheelIdx = 0; WheelIdx < NumWheels; ++WheelIdx)
				{
					NumLength += Vehicle->GetSuspension(WheelIdx).GetAveragingNum();
				}
				
				SuspensionLastSpringLength.SetNum(NumWheels);
				SuspensionLastDisplacement.SetNum(NumWheels);
				SuspensionAveragedCount.SetNum(NumWheels);
				SuspensionAveragedNum.SetNum(NumWheels);
				SuspensionAveragedLength.SetNum(NumLength);
				WheelsAngularPosition.SetNum(NumWheels);
				WheelsOmega.SetNum(NumWheels);
				
				int32 LengthCount = 0;
				for (int32 WheelIdx = 0; WheelIdx < Vehicle->Wheels.Num(); ++WheelIdx)
				{
					Chaos::FSimpleSuspensionSim& Suspension = Vehicle->GetSuspension(WheelIdx);
					SuspensionLastSpringLength[WheelIdx] = Suspension.GetLastSpringLength();
					SuspensionLastDisplacement[WheelIdx] = Suspension.GetLastDisplacement();
					SuspensionAveragedCount[WheelIdx] = Suspension.GetAveragingCount();
					SuspensionAveragedNum[WheelIdx] = Suspension.GetAveragingNum();
				
					for (int32 LengthIdx = 0; LengthIdx < Suspension.GetAveragingNum(); ++LengthIdx)
					{
						SuspensionAveragedLength[LengthCount++] = Suspension.GetAveragingLength(LengthIdx);
					}
				
					Chaos::FSimpleWheelSim& Wheel = Vehicle->GetWheel(WheelIdx);
					WheelsOmega[WheelIdx] = Wheel.Omega;
					WheelsAngularPosition[WheelIdx] = Wheel.AngularPosition;
				}
			}
		}
	}
}
void FNetworkVehicleStates::InterpolateData(const FNetworkPhysicsData& MinData, const FNetworkPhysicsData& MaxData)
{
	const FNetworkVehicleStates& MinState = static_cast<const FNetworkVehicleStates&>(MinData);
	const FNetworkVehicleStates& MaxState = static_cast<const FNetworkVehicleStates&>(MaxData);

	const float LerpFactor = (LocalFrame - MinState.LocalFrame) / (MaxState.LocalFrame - MinState.LocalFrame);

	StateLastVelocity = FMath::Lerp(MinState.StateLastVelocity, MaxState.StateLastVelocity, LerpFactor);
	EngineOmega = FMath::Lerp(MinState.EngineOmega, MaxState.EngineOmega, LerpFactor);

	int32 NumWheels = FMath::Min(MinState.WheelsOmega.Num(), MaxState.WheelsOmega.Num());

	WheelsOmega.SetNum(NumWheels, EAllowShrinking::No);
	WheelsAngularPosition.SetNum(NumWheels, EAllowShrinking::No);

	SuspensionLastDisplacement.SetNum(NumWheels, EAllowShrinking::No);
	SuspensionLastSpringLength.SetNum(NumWheels, EAllowShrinking::No);
	SuspensionAveragedCount.SetNum(NumWheels, EAllowShrinking::No);
	SuspensionAveragedNum.SetNum(NumWheels, EAllowShrinking::No);

	int32 NumLength = 0;
	for (int32 WheelIdx = 0; WheelIdx < NumWheels; ++WheelIdx)
	{
		WheelsOmega[WheelIdx] = FMath::Lerp(MinState.WheelsOmega[WheelIdx], MaxState.WheelsOmega[WheelIdx], LerpFactor);
		WheelsAngularPosition[WheelIdx] = FMath::Lerp(MinState.WheelsAngularPosition[WheelIdx], MaxState.WheelsAngularPosition[WheelIdx], LerpFactor);

		SuspensionLastDisplacement[WheelIdx] = FMath::Lerp(MinState.SuspensionLastDisplacement[WheelIdx], MaxState.SuspensionLastDisplacement[WheelIdx], LerpFactor);
		SuspensionLastSpringLength[WheelIdx] = FMath::Lerp(MinState.SuspensionLastSpringLength[WheelIdx], MaxState.SuspensionLastSpringLength[WheelIdx], LerpFactor);
		SuspensionAveragedCount[WheelIdx] = FMath::Min(MinState.SuspensionAveragedCount[WheelIdx], MaxState.SuspensionAveragedCount[WheelIdx]);
		SuspensionAveragedNum[WheelIdx] = FMath::Min(MinState.SuspensionAveragedNum[WheelIdx], MaxState.SuspensionAveragedNum[WheelIdx]);

		NumLength += SuspensionAveragedNum[WheelIdx];
	}
	SuspensionAveragedLength.SetNum(NumLength, EAllowShrinking::No);
	for (int32 LengthIdx = 0; LengthIdx < NumLength; ++LengthIdx)
	{
		SuspensionAveragedLength[LengthIdx] = FMath::Lerp(MinState.SuspensionAveragedLength[LengthIdx], MaxState.SuspensionAveragedLength[LengthIdx], LerpFactor);
	}
}

const FString FNetworkVehicleStates::DebugData()
{
	return FString::Printf(TEXT("FNetworkVehicleStates | EngineOmega = %f"), EngineOmega);
}