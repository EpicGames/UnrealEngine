// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVehicleMovementComponent.h"
#include "BodySetupCore.h"
#include "EngineGlobals.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/PlayerController.h"
#include "Engine/Engine.h"
#include "Engine/SkinnedAsset.h"
#include "CanvasItem.h"
#include "Engine/Canvas.h"
#include "Components/StaticMeshComponent.h"
#include "Components/SkinnedMeshComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "DrawDebugHelpers.h"
#include "UObject/FrameworkObjectVersion.h"
#include "Net/UnrealNetwork.h"
#include "VehicleAnimationInstance.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "Physics/PhysicsFiltering.h"
#include "Physics/PhysicsInterfaceUtils.h"
#include "GameFramework/PawnMovementComponent.h"
#include "Logging/MessageLog.h"
#include "DisplayDebugHelpers.h"
#include "Chaos/ChaosEngineInterface.h"
#include "Chaos/PBDJointConstraintData.h"
#include "Chaos/DebugDrawQueue.h"

#include "ChaosVehicleManager.h"
#include "SimpleVehicle.h"

#include "AI/Navigation/AvoidanceManager.h"
#include "PhysicalMaterials/PhysicalMaterial.h"
#include "GameFramework/HUD.h"
#include "PhysicsEngine/PhysicsSettings.h"

#include "Physics/Experimental/PhysScene_Chaos.h"
#include "Chaos/Particle/ParticleUtilities.h"
#include "Chaos/ParticleHandleFwd.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "PBDRigidsSolver.h"

#if UE_WITH_REMOTE_OBJECT_HANDLE
#include "UObject/UObjectMigrationContext.h"
#endif

#define LOCTEXT_NAMESPACE "UVehicleMovementComponent"

#if VEHICLE_DEBUGGING_ENABLED
UE_DISABLE_OPTIMIZATION
#endif

DEFINE_LOG_CATEGORY(LogVehicle);



FVehicleDebugParams GVehicleDebugParams;

FAutoConsoleVariableRef CVarChaosVehiclesShowCOM(TEXT("p.Vehicle.ShowCOM"), GVehicleDebugParams.ShowCOM, TEXT("Enable/Disable Center Of Mass Debug Visualisation."));
FAutoConsoleVariableRef CVarChaosVehiclesShowModelAxis(TEXT("p.Vehicle.ShowModelOrigin"), GVehicleDebugParams.ShowModelOrigin, TEXT("Enable/Disable Model Origin Visualisation."));
FAutoConsoleVariableRef CVarChaosVehiclesShowAllForces(TEXT("p.Vehicle.ShowAllForces"), GVehicleDebugParams.ShowAllForces, TEXT("Enable/Disable Force Visualisation."));
FAutoConsoleVariableRef CVarChaosVehiclesAerofoilForces(TEXT("p.Vehicle.ShowAerofoilForces"), GVehicleDebugParams.ShowAerofoilForces, TEXT("Enable/Disable Aerofoil Force Visualisation."));
FAutoConsoleVariableRef CVarChaosVehiclesAerofoilSurface(TEXT("p.Vehicle.ShowAerofoilSurface"), GVehicleDebugParams.ShowAerofoilSurface, TEXT("Enable/Disable a very approximate visualisation of where the Aerofoil surface is located and its orientation."));
FAutoConsoleVariableRef CVarChaosVehiclesDisableTorqueControl(TEXT("p.Vehicle.DisableTorqueControl"), GVehicleDebugParams.DisableTorqueControl, TEXT("Enable/Disable Direct Torque Control."));
FAutoConsoleVariableRef CVarChaosVehiclesDisableStabilizeControl(TEXT("p.Vehicle.DisableStabilizeControl"), GVehicleDebugParams.DisableStabilizeControl, TEXT("Enable/Disable Position Stabilization Control."));
FAutoConsoleVariableRef CVarChaosVehiclesDisableAerodynamics(TEXT("p.Vehicle.DisableAerodynamics"), GVehicleDebugParams.DisableAerodynamics, TEXT("Enable/Disable Aerodynamic Forces Drag/Downforce."));
FAutoConsoleVariableRef CVarChaosVehiclesDisableAerofoils(TEXT("p.Vehicle.DisableAerofoils"), GVehicleDebugParams.DisableAerofoils, TEXT("Enable/Disable Aerofoil Forces."));
FAutoConsoleVariableRef CVarChaosVehiclesDisableThrusters(TEXT("p.Vehicle.DisableThrusters"), GVehicleDebugParams.DisableThrusters, TEXT("Enable/Disable Thruster Forces."));
FAutoConsoleVariableRef CVarChaosVehiclesBatchQueries(TEXT("p.Vehicle.BatchQueries"), GVehicleDebugParams.BatchQueries, TEXT("Enable/Disable Batching Of Suspension Raycasts."));
FAutoConsoleVariableRef CVarChaosVehiclesCacheTraceOverlap(TEXT("p.Vehicle.CacheTraceOverlap"), GVehicleDebugParams.CacheTraceOverlap, TEXT("Enable/Disable Caching Of Suspension Trace Overlap Test Optimization (only valid when BatchQueries enabled)."));
FAutoConsoleVariableRef CVarChaosVehiclesForceDebugScaling(TEXT("p.Vehicle.SetForceDebugScaling"), GVehicleDebugParams.ForceDebugScaling, TEXT("Set Scaling For Force Visualisation."));
FAutoConsoleVariableRef CVarChaosVehiclesSleepCounterThreshold(TEXT("p.Vehicle.SleepCounterThreshold"), GVehicleDebugParams.SleepCounterThreshold, TEXT("Set The Sleep Counter Iteration Threshold."));
FAutoConsoleVariableRef CVarChaosVehiclesDisableVehicleSleep(TEXT("p.Vehicle.DisableVehicleSleep"), GVehicleDebugParams.DisableVehicleSleep, TEXT("Disable Vehicle Agressive Sleeping."));
FAutoConsoleVariableRef CVarChaosVehiclesSetMaxMPH(TEXT("p.Vehicle.SetMaxMPH"), GVehicleDebugParams.SetMaxMPH, TEXT("Set a top speed in MPH (affects all vehicles)."));
FAutoConsoleVariableRef CVarChaosVehiclesEnableMultithreading(TEXT("p.Vehicle.EnableMultithreading"), GVehicleDebugParams.EnableMultithreading, TEXT("Enable multi-threading of vehicle updates."));
FAutoConsoleVariableRef CVarChaosVehiclesControlInputWakeTolerance(TEXT("p.Vehicle.ControlInputWakeTolerance"), GVehicleDebugParams.ControlInputWakeTolerance, TEXT("Set the control input wake tolerance."));


void FVehicleState::CaptureState(const FBodyInstance* TargetInstance, float GravityZ, float DeltaTime)
{
	if (TargetInstance)
	{

		VehicleWorldTransform = TargetInstance->GetUnrealWorldTransform();
		VehicleWorldVelocity = TargetInstance->GetUnrealWorldVelocity();
		VehicleWorldAngularVelocity = TargetInstance->GetUnrealWorldAngularVelocityInRadians();
		VehicleWorldCOM = TargetInstance->GetCOMPosition();
		WorldVelocityNormal = VehicleWorldVelocity.GetSafeNormal();

		VehicleUpAxis = VehicleWorldTransform.GetUnitAxis(EAxis::Z);
		VehicleForwardAxis = VehicleWorldTransform.GetUnitAxis(EAxis::X);
		VehicleRightAxis = VehicleWorldTransform.GetUnitAxis(EAxis::Y);

		VehicleLocalVelocity = VehicleWorldTransform.InverseTransformVector(VehicleWorldVelocity);
		LocalAcceleration = (VehicleLocalVelocity - LastFrameVehicleLocalVelocity) / DeltaTime;
		LocalGForce = LocalAcceleration / FMath::Abs(GravityZ);
		LastFrameVehicleLocalVelocity = VehicleLocalVelocity;

		ForwardSpeed = FVector::DotProduct(VehicleWorldVelocity, VehicleForwardAxis);
		ForwardsAcceleration = LocalAcceleration.X;
	}
}

void FVehicleState::CaptureState(const Chaos::FRigidBodyHandle_Internal* Handle, float GravityZ, float DeltaTime)
{
	if (Handle)
	{
		const FTransform WorldTM(Handle->R(), Handle->X());
		VehicleWorldTransform = WorldTM;
		VehicleWorldVelocity = Handle->V();
		VehicleWorldAngularVelocity = Handle->W();
		VehicleWorldCOM = Handle->CenterOfMass();
		WorldVelocityNormal = VehicleWorldVelocity.GetSafeNormal();

		VehicleUpAxis = VehicleWorldTransform.GetUnitAxis(EAxis::Z);
		VehicleForwardAxis = VehicleWorldTransform.GetUnitAxis(EAxis::X);
		VehicleRightAxis = VehicleWorldTransform.GetUnitAxis(EAxis::Y);

		VehicleLocalVelocity = VehicleWorldTransform.InverseTransformVector(VehicleWorldVelocity);
		LocalAcceleration = (VehicleLocalVelocity - LastFrameVehicleLocalVelocity) / DeltaTime;
		LocalGForce = LocalAcceleration / FMath::Abs(GravityZ);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (Chaos::FPhysicsSolverBase::IsNetworkPhysicsPredictionEnabled() && Chaos::FPhysicsSolverBase::CanDebugNetworkPhysicsPrediction())
		{
			UE_LOGF(LogVehicle, Log, "Vehicle state last velocity = %ls | World Transform = %ls | World Velocity = %ls",
				*LastFrameVehicleLocalVelocity.ToString(), *VehicleWorldTransform.ToString(), *VehicleWorldVelocity.ToString());
		}
#endif
		LastFrameVehicleLocalVelocity = VehicleLocalVelocity;

		ForwardSpeed = FVector::DotProduct(VehicleWorldVelocity, VehicleForwardAxis);
		ForwardsAcceleration = LocalAcceleration.X;


	}
}


/**
 * UChaosVehicleSimulation
 */
void UChaosVehicleSimulation::TickVehicle(UWorld* WorldIn, float DeltaTime, const FChaosVehicleAsyncInput& InputData, FChaosVehicleAsyncOutput& OutputData, Chaos::FRigidBodyHandle_Internal* Handle)
{
	World = WorldIn;
	RigidHandle = Handle;

	// movement updates and replication
	if (World && RigidHandle)
	{
#if DEBUG_NETWORK_PHYSICS
		if (WorldIn->IsNetMode(NM_ListenServer) || WorldIn->IsNetMode(NM_DedicatedServer))
		{
			UE_LOGF(LogTemp, Log, "SERVER | PT | TickVehicle | Async tick vehicle with inputs at frame %d : Throttle = %f Brake = %f Roll = %f Pitch = %f Yaw = %f Steering = %f Handbrake = %f Sleeping = %d",
				InputData.PhysicsInputs.NetworkInputs.LocalFrame, VehicleInputs.ThrottleInput, VehicleInputs.BrakeInput, VehicleInputs.RollInput, VehicleInputs.PitchInput,
				VehicleInputs.YawInput, VehicleInputs.SteeringInput, VehicleInputs.HandbrakeInput, VehicleState.bSleeping);
		}
		else if(WorldIn->IsNetMode(NM_Client))
		{
			UE_LOGF(LogTemp, Log, "CLIENT | PT | TickVehicle | Async tick vehicle with inputs at frame %d : Throttle = %f Brake = %f Roll = %f Pitch = %f Yaw = %f Steering = %f Handbrake = %f Sleeping = %d",
				InputData.PhysicsInputs.NetworkInputs.LocalFrame, VehicleInputs.ThrottleInput, VehicleInputs.BrakeInput, VehicleInputs.RollInput, VehicleInputs.PitchInput,
				VehicleInputs.YawInput, VehicleInputs.SteeringInput, VehicleInputs.HandbrakeInput, VehicleState.bSleeping);
		}
#endif

		if (!VehicleState.bSleeping)
		{
			if (CanSimulate() && Handle)
			{
				// Update the vehicle/wheels... states
				UpdateState(DeltaTime, InputData, Handle);

				// Apply the controls inputs
				ApplyInput(InputData.PhysicsInputs.NetworkInputs.VehicleInputs, DeltaTime);

				// Update the simulation forces/impulses...
				UpdateSimulation(DeltaTime, InputData, Handle);
				FillOutputState(OutputData);
			}
		}
	}


#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	DrawDebug3D();
#endif
}

void UChaosVehicleSimulation::ApplyDeferredForces(Chaos::FRigidBodyHandle_Internal* Handle)
{
	DeferredForces.Apply(Handle);
}

void UChaosVehicleSimulation::UpdateState(float DeltaTime, const FChaosVehicleAsyncInput& InputData, Chaos::FRigidBodyHandle_Internal* Handle)
{
	VehicleState.CaptureState(Handle, InputData.PhysicsInputs.GravityZ, DeltaTime);
}

void UChaosVehicleSimulation::UpdateSimulation(float DeltaTime, const FChaosVehicleAsyncInput& InputData, Chaos::FRigidBodyHandle_Internal* Handle)
{
	ApplyAerodynamics(DeltaTime);
	ApplyAerofoilForces(DeltaTime);
	ApplyThrustForces(DeltaTime);
	ApplyTorqueControl(DeltaTime, InputData);
}

void UChaosVehicleSimulation::FillOutputState(FChaosVehicleAsyncOutput& Output)
{}

/** Pass control Input to the vehicle systems */
void UChaosVehicleSimulation::ApplyInput(const FControlInputs& ControlInputs, float DeltaTime)
{
	for (int AerofoilIdx = 0; AerofoilIdx < PVehicle->Aerofoils.Num(); AerofoilIdx++)
	{
		Chaos::FAerofoil& Aerofoil = PVehicle->GetAerofoil(AerofoilIdx);
		switch (Aerofoil.Setup().Type)
		{
		case Chaos::EAerofoilType::Rudder:
			Aerofoil.SetControlSurface(-ControlInputs.YawInput);
			break;

		case Chaos::EAerofoilType::Elevator:
			Aerofoil.SetControlSurface(ControlInputs.PitchInput);
			break;

		case Chaos::EAerofoilType::Wing:
			if (Aerofoil.Setup().Offset.Y < 0.0f)
			{
				Aerofoil.SetControlSurface(ControlInputs.RollInput);
			}
			else
			{
				Aerofoil.SetControlSurface(-ControlInputs.RollInput);
			}
			break;
		}
	}

	for (int Thrusterdx = 0; Thrusterdx < PVehicle->Thrusters.Num(); Thrusterdx++)
	{
		Chaos::FSimpleThrustSim& Thruster = PVehicle->GetThruster(Thrusterdx);

		Thruster.SetThrottle(ControlInputs.ThrottleInput);

		switch (Thruster.Setup().Type)
		{
		case Chaos::EThrustType::HelicopterRotor:
		{
			Thruster.SetPitch(ControlInputs.PitchInput);
			Thruster.SetRoll(ControlInputs.RollInput);
		}
		break;

		case Chaos::EThrustType::Rudder:
		{
			Thruster.SetYaw(-ControlInputs.YawInput - ControlInputs.SteeringInput);
		}
		break;

		case Chaos::EThrustType::Elevator:
		{
			Thruster.SetPitch(ControlInputs.PitchInput);
		}
		break;

		case Chaos::EThrustType::Wing:
		{
			if (Thruster.Setup().Offset.Y < 0.0f)
			{
				Thruster.SetRoll(ControlInputs.RollInput);
			}
			else
			{
				Thruster.SetRoll(-ControlInputs.RollInput);
			}
		}
		break;

		}

	}

}

void UChaosVehicleSimulation::ApplyAerodynamics(float DeltaTime)
{
	if (!GVehicleDebugParams.DisableAerodynamics)
	{
		// This force applied all the time whether the vehicle is on the ground or not
		Chaos::FSimpleAerodynamicsSim& PAerodynamics = PVehicle->GetAerodynamics();
		FVector LocalDragLiftForce = (PAerodynamics.GetCombinedForces(Chaos::CmToM(VehicleState.ForwardSpeed))) * Chaos::MToCmScaling();
		FVector WorldLiftDragForce = VehicleState.VehicleWorldTransform.TransformVector(LocalDragLiftForce);
		AddForce(WorldLiftDragForce);
	}
}

FVector GetWorldVelocityAtPoint(const Chaos::FRigidBodyHandle_Internal* RigidHandle, const FVector& WorldLocation)
{
	if (RigidHandle)
	{
		const Chaos::FVec3 COM = RigidHandle ? Chaos::FParticleUtilitiesGT::GetCoMWorldPosition(RigidHandle) : (Chaos::FVec3)Chaos::FParticleUtilitiesGT::GetActorWorldTransform(RigidHandle).GetTranslation();
		const Chaos::FVec3 Diff = WorldLocation - COM;
		return RigidHandle->V() - Chaos::FVec3::CrossProduct(Diff, RigidHandle->W());
	}
	else
	{
		return FVector::ZeroVector;
	}
}

void UChaosVehicleSimulation::ApplyAerofoilForces(float DeltaTime)
{
	if (GVehicleDebugParams.DisableAerofoils || RigidHandle == nullptr)
		return;

	TArray<FVector> VelocityLocal;
	TArray<FVector> VelocityWorld;
	VelocityLocal.SetNum(PVehicle->Aerofoils.Num());
	VelocityWorld.SetNum(PVehicle->Aerofoils.Num());

	float Altitude = VehicleState.VehicleWorldTransform.GetLocation().Z;

	// Work out velocity at each aerofoil before applying any forces so there's no bias on the first ones processed
	for (int AerofoilIdx = 0; AerofoilIdx < PVehicle->Aerofoils.Num(); AerofoilIdx++)
	{
		FVector WorldLocation = VehicleState.VehicleWorldTransform.TransformPosition(PVehicle->GetAerofoil(AerofoilIdx).Setup().Offset * Chaos::MToCmScaling());
		VelocityWorld[AerofoilIdx] = GetWorldVelocityAtPoint(RigidHandle, WorldLocation);
		VelocityLocal[AerofoilIdx] = VehicleState.VehicleWorldTransform.InverseTransformVector(VelocityWorld[AerofoilIdx]);
	}

	for (int AerofoilIdx = 0; AerofoilIdx < PVehicle->Aerofoils.Num(); AerofoilIdx++)
	{
		Chaos::FAerofoil& Aerofoil = PVehicle->GetAerofoil(AerofoilIdx);

		FVector LocalForce = Aerofoil.GetForce(VehicleState.VehicleWorldTransform, VelocityLocal[AerofoilIdx] * Chaos::CmToMScaling(), Chaos::CmToM(Altitude), DeltaTime);

		FVector WorldForce = VehicleState.VehicleWorldTransform.TransformVector(LocalForce);
		FVector WorldLocation = VehicleState.VehicleWorldTransform.TransformPosition(Aerofoil.GetCenterOfLiftOffset() * Chaos::MToCmScaling());
		AddForceAtPosition(WorldForce * Chaos::MToCmScaling(), WorldLocation);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		FVector WorldAxis = VehicleState.VehicleWorldTransform.TransformVector(FVector::CrossProduct(FVector(1, 0, 0), Aerofoil.Setup().UpAxis));
		if (GVehicleDebugParams.ShowAerofoilSurface)
		{
			Chaos::FDebugDrawQueue::GetInstance().DrawDebugLine(WorldLocation - WorldAxis * 150.0f, WorldLocation + WorldAxis * 150.0f, FColor::Black, false, -1.f, 0, 5.f);
		}
		if (GVehicleDebugParams.ShowAerofoilForces)
		{
			Chaos::FDebugDrawQueue::GetInstance().DrawDebugLine(WorldLocation, WorldLocation + WorldForce * GVehicleDebugParams.ForceDebugScaling, FColor::Green, false, -1.f, 0, 16.f);
		}
#endif
	}

}


void UChaosVehicleSimulation::ApplyThrustForces(float DeltaTime)
{
	if (GVehicleDebugParams.DisableThrusters || RigidHandle == nullptr)
		return;

	for (int ThrusterIdx = 0; ThrusterIdx < PVehicle->Thrusters.Num(); ThrusterIdx++)
	{
		Chaos::FSimpleThrustSim& Thruster = PVehicle->GetThruster(ThrusterIdx);

		FVector COM_Offset = RigidHandle->CenterOfMass();
		COM_Offset.Z = 0.0f;
		Thruster.SetWorldVelocity(VehicleState.VehicleWorldVelocity);

		Thruster.Simulate(DeltaTime);
		FVector ThrustWorldLocation = VehicleState.VehicleWorldTransform.TransformPosition(Thruster.GetThrustLocation() + COM_Offset);
		FVector ThrustForce = VehicleState.VehicleWorldTransform.TransformVector(Thruster.GetThrustForce());

		AddForceAtPosition(ThrustForce, ThrustWorldLocation);
	}

}


void UChaosVehicleSimulation::ApplyTorqueControl(float DeltaTime, const FChaosVehicleAsyncInput& InputData)
{
	if (!PVehicle->HasTorqueControlSetup())
	{
		return;
	}

	const FControlInputs& ControlInputsPT = InputData.PhysicsInputs.NetworkInputs.VehicleInputs;

	if (!GVehicleDebugParams.DisableTorqueControl && RigidHandle)
	{
		FVector TotalTorque = FVector::ZeroVector;
		if (PVehicle->HasTorqueControlSetup() && PVehicle->GetTorqueControl().Setup().Enabled)
		{
			const Chaos::FTargetRotationControlConfig& TargetRotationControl = PVehicle->GetTargetRotationControl().Setup();
			auto ComputeTorque = [](const FVector& TargetUp, const FVector& CurrentUp, const FVector& AngVelocityWorld, float Stiffness, float Damping, float MaxAccel) -> FVector
			{
				const FQuat CurUpToTargetUp = FQuat::FindBetweenNormals(CurrentUp, TargetUp);
				const FVector Axis = CurUpToTargetUp.GetRotationAxis();
				const float Angle = CurUpToTargetUp.GetAngle();

				float Strength = (Angle * Stiffness - FVector::DotProduct(AngVelocityWorld, Axis) * Damping);
				Strength = FMath::Clamp(Strength, -MaxAccel, MaxAccel);
				const FVector Torque = Axis * Strength;
				return Torque;
			};


			FVector TargetUp = FVector(0.f, 0.f, 1.f);
			float RollMaxAngleRadians = Chaos::DegToRad(TargetRotationControl.RollMaxAngle);
			float PitchMaxAngleRadians = Chaos::DegToRad(TargetRotationControl.PitchMaxAngle);
			float Speed = FMath::Min(Chaos::CmToM(VehicleState.ForwardSpeed), 20.0f); // cap here

			float SpeeScaledRollAmount = 1.0f;
			float TargetRoll = 0.f;
			if (TargetRotationControl.bRollVsSpeedEnabled)
			{
				if ((PVehicle->Wheels.Num() > 0) && PVehicle->Wheels[0].InContact())
				{
					TargetRoll = ControlInputsPT.SteeringInput * TargetRotationControl.RollControlScaling * (Speed * Speed) * DeltaTime * 60.0f;
				}
			}
			else
			{
				TargetRoll = ControlInputsPT.SteeringInput * TargetRotationControl.RollControlScaling;
			}

			FVector Rt = VehicleState.VehicleRightAxis * FMath::Max(FMath::Min(TargetRoll, RollMaxAngleRadians), -RollMaxAngleRadians);
			FVector Pt = VehicleState.VehicleForwardAxis * FMath::Max(FMath::Min(ControlInputsPT.PitchInput * TargetRotationControl.PitchControlScaling, PitchMaxAngleRadians), -PitchMaxAngleRadians);

			FVector UseUp = TargetUp + Rt + Pt;
			UseUp.Normalize();

			TargetUp = UseUp;

			const FVector UpVector = VehicleState.VehicleUpAxis;
			const FVector AngVelocityWorld = VehicleState.VehicleWorldAngularVelocity;

			const FVector AirControlTorque = ComputeTorque(TargetUp, UpVector, AngVelocityWorld, TargetRotationControl.RotationStiffness, TargetRotationControl.RotationDamping, TargetRotationControl.MaxAccel);
			const FVector ForwardVector = VehicleState.VehicleForwardAxis;
			const FVector RightVector = VehicleState.VehicleRightAxis;

			const float RollAirControl = FVector::DotProduct(AirControlTorque, ForwardVector);
			const float PitchAirControl = FVector::DotProduct(AirControlTorque, RightVector);
			const float YawAirControl = FVector::DotProduct(AirControlTorque, UpVector);

			TotalTorque = RollAirControl * ForwardVector * TargetRotationControl.AutoCentreRollStrength
				+ YawAirControl * UpVector * TargetRotationControl.AutoCentreYawStrength
				+ PitchAirControl * RightVector * TargetRotationControl.AutoCentrePitchStrength;
		}

		if (PVehicle->HasTorqueControlSetup() && PVehicle->GetTorqueControl().Setup().Enabled)
		{
			const Chaos::FTorqueControlConfig& TorqueControl = PVehicle->GetTorqueControl().Setup();

			TotalTorque -= VehicleState.VehicleForwardAxis * ControlInputsPT.RollInput * TorqueControl.RollTorqueScaling;
			TotalTorque += VehicleState.VehicleRightAxis * ControlInputsPT.PitchInput * TorqueControl.PitchTorqueScaling;
			TotalTorque += VehicleState.VehicleUpAxis * ControlInputsPT.YawInput * TorqueControl.YawTorqueScaling;
			TotalTorque += VehicleState.VehicleUpAxis * ControlInputsPT.RollInput * TorqueControl.YawFromRollTorqueScaling;

			// slowing rotation effect
			FVector DampingTorque = (VehicleState.VehicleWorldAngularVelocity) * TorqueControl.RotationDamping;

			// combined world torque
			TotalTorque -= DampingTorque;
		}

		AddTorqueInRadians(TotalTorque, true, true);
	}


	if (!GVehicleDebugParams.DisableStabilizeControl && PVehicle->HasStabilizeControlSetup() && PVehicle->GetStabilizeControl().Setup().Enabled && RigidHandle)
	{
		const Chaos::FStabilizeControlConfig& StabilizeControl = PVehicle->GetStabilizeControl().Setup();

		// try to cancel out velocity on Z axis
		FVector CorrectionalForce = FVector::ZeroVector;
		{
			bool MaintainAltitude = true;
			if (MaintainAltitude)
			{
				CorrectionalForce.Z = -StabilizeControl.AltitudeHoldZ * VehicleState.VehicleWorldVelocity.Z / DeltaTime;
			}
		}

		// try to cancel out velocity on X/Y plane
		// #todo: Will break helicopter setup??if (FMath::Abs(RollInput) < SMALL_NUMBER && FMath::Abs(PitchInput) < SMALL_NUMBER)
		{
			CorrectionalForce.X = -StabilizeControl.PositionHoldXY * VehicleState.VehicleWorldVelocity.X / DeltaTime;
			CorrectionalForce.Y = -StabilizeControl.PositionHoldXY * VehicleState.VehicleWorldVelocity.Y / DeltaTime;
		}
		AddForce(CorrectionalForce);
	}

}

void UChaosVehicleSimulation::DrawDebug3D()
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (RigidHandle == nullptr)
	{
		return;
	}

	const FTransform BodyTransform = VehicleState.VehicleWorldTransform;

	if (GVehicleDebugParams.ShowCOM)
	{
		const Chaos::FVec3 COMWorld = Chaos::FParticleUtilitiesGT::GetCoMWorldPosition(RigidHandle);
		Chaos::FDebugDrawQueue::GetInstance().DrawDebugCoordinateSystem(COMWorld, FRotator(BodyTransform.GetRotation()), 200.f, false, -1.f, 0, 2.f);
	}

	if (GVehicleDebugParams.ShowModelOrigin)
	{
		Chaos::FDebugDrawQueue::GetInstance().DrawDebugCoordinateSystem(BodyTransform.GetLocation(), FRotator(BodyTransform.GetRotation()), 200.f, false, -1.f, 0, 2.f);
	}
#endif
}

void UChaosVehicleSimulation::AddForce(const FVector& Force, bool bAllowSubstepping, bool bAccelChange)
{
	DeferredForces.Add(FDeferredForces::FApplyForceData(Force, bAllowSubstepping, bAccelChange));

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (GVehicleDebugParams.ShowAllForces)
	{
		FVector Position = RigidHandle->X();
		Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(Position, Position + Force * GVehicleDebugParams.ForceDebugScaling
			, 20.f, FColor::Blue, false, 0, 0, 2.f);
	}
#endif	
}

void UChaosVehicleSimulation::AddForceAtPosition(const FVector& Force, const FVector& Position, bool bAllowSubstepping, bool bIsLocalForce)
{
	DeferredForces.Add(FDeferredForces::FApplyForceAtPositionData(Force, Position, bAllowSubstepping, bIsLocalForce));

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (GVehicleDebugParams.ShowAllForces)
	{
		Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(Position, Position + Force * GVehicleDebugParams.ForceDebugScaling
			, 20.f, FColor::Blue, false, 0, 0, 2.f);
	}
#endif
}

void UChaosVehicleSimulation::AddImpulse(const FVector& Impulse, bool bVelChange)
{
	DeferredForces.Add(FDeferredForces::FAddImpulseData(Impulse, bVelChange));

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (GVehicleDebugParams.ShowAllForces)
	{
		FVector Position = VehicleState.VehicleWorldCOM;
		Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(Position, Position + Impulse * GVehicleDebugParams.ForceDebugScaling
			, 20.f, FColor::Red, false, 0, 0, 2.f);
	}
#endif
	
}

void UChaosVehicleSimulation::AddImpulseAtPosition(const FVector& Impulse, const FVector& Position)
{
	DeferredForces.Add(FDeferredForces::FAddImpulseAtPositionData(Impulse, Position));

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (GVehicleDebugParams.ShowAllForces)
	{
		Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(Position, Position + Impulse * GVehicleDebugParams.ForceDebugScaling
			, 20.f, FColor::Red, false, 0, 0, 2.f);
	}
#endif

}

void UChaosVehicleSimulation::AddTorqueInRadians(const FVector& Torque, bool bAllowSubstepping /*= true*/, bool bAccelChange /*= false*/)
{
	DeferredForces.Add(FDeferredForces::FAddTorqueInRadiansData(Torque, bAllowSubstepping, bAccelChange));
}

void UChaosVehicleSimulation::InitializeWheel(int WheelIndex, const Chaos::FSimpleWheelConfig* InWheelSetup)
{
	if (PVehicle->IsValid() && InWheelSetup && WheelIndex < PVehicle->Wheels.Num())
	{
		PVehicle->Wheels[WheelIndex].SetupPtr = InWheelSetup;
		PVehicle->Wheels[WheelIndex].SetWheelRadius(InWheelSetup->WheelRadius);
	}
}

void UChaosVehicleSimulation::InitializeSuspension(int WheelIndex, const Chaos::FSimpleSuspensionConfig* InSuspensionSetup)
{
	if (PVehicle->IsValid() && InSuspensionSetup && WheelIndex < PVehicle->Suspension.Num())
	{
		PVehicle->Suspension[WheelIndex].SetupPtr = InSuspensionSetup;
	}
}

/**
 * UChaosVehicleMovementComponent
 */
UChaosVehicleMovementComponent::UChaosVehicleMovementComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bReverseAsBrake = true;
	bParkEnabled = false;
	Mass = 1500.0f;
	ChassisWidth = 180.f;
	ChassisHeight = 140.f;
	DragCoefficient = 0.3f;
	DownforceCoefficient = 0.3f;
	InertiaTensorScale = FVector( 1.0f, 1.0f, 1.0f );
	SleepThreshold = 10.0f;
	SleepSlopeLimit = 0.866f;	// 30 degrees, Cos(30)

	TorqueControl.InitDefaults();
	TargetRotationControl.InitDefaults();
	StabilizeControl.InitDefaults();

	AngErrorAccumulator = 0.0f;
	TargetGear = 0;

	PrevSteeringInput = 0.0f;
	PrevReplicatedSteeringInput = 0.0f;

	bRequiresControllerForInputs = true;
	IdleBrakeInput = 0.0f;
	StopThreshold = 10.0f; 
	WrongDirectionThreshold = 100.f;
	ThrottleInputRate.RiseRate = 6.0f;
	ThrottleInputRate.FallRate = 10.0f;
	ThrottleInputRate.InputCurveFunction = EInputFunctionType::LinearFunction;
	BrakeInputRate.RiseRate = 6.0f;
	BrakeInputRate.FallRate = 10.0f;
	BrakeInputRate.InputCurveFunction = EInputFunctionType::LinearFunction;
	SteeringInputRate.RiseRate = 2.5f;
	SteeringInputRate.FallRate = 5.0f;
	SteeringInputRate.InputCurveFunction = EInputFunctionType::SquaredFunction;
	HandbrakeInputRate.RiseRate = 12.0f;
	HandbrakeInputRate.FallRate = 12.0f;
	PitchInputRate.RiseRate = 6.0f;
	PitchInputRate.FallRate = 10.0f;
	PitchInputRate.InputCurveFunction = EInputFunctionType::LinearFunction;
	RollInputRate.RiseRate = 6.0f;
	RollInputRate.FallRate = 10.0f;
	RollInputRate.InputCurveFunction = EInputFunctionType::LinearFunction;
	YawInputRate.RiseRate = 6.0f;
	YawInputRate.FallRate = 10.0f;
	YawInputRate.InputCurveFunction = EInputFunctionType::LinearFunction;
	TransmissionType = Chaos::ETransmissionType::Automatic;

	SetIsReplicatedByDefault(true);
	bUsingNetworkPhysicsPrediction = Chaos::FPhysicsSolverBase::IsNetworkPhysicsPredictionEnabled();

	AHUD::OnShowDebugInfo.AddUObject(this, &UChaosVehicleMovementComponent::ShowDebugInfo);

	if (bUsingNetworkPhysicsPrediction)
	{
		static const FName NetworkPhysicsComponentName(TEXT("PC_NetworkPhysicsComponent"));

		NetworkPhysicsComponent = CreateDefaultSubobject<UNetworkPhysicsComponent, UNetworkPhysicsComponent>(NetworkPhysicsComponentName);
		NetworkPhysicsComponent->SetNetAddressable(); // Make DSO components net addressable
		NetworkPhysicsComponent->SetIsReplicated(true);
	}
}

// public

void UChaosVehicleMovementComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	// Custom serialization goes here...
#if UE_WITH_REMOTE_OBJECT_HANDLE
	if (Ar.IsMigratingRemoteObjects())
	{
		if (Ar.IsLoading())
		{
			bIsMigrated = true;
		}
		MigratePhysicsState(Ar);
	}
#endif
}

#if UE_WITH_REMOTE_OBJECT_HANDLE
void UChaosVehicleMovementComponent::MigratePhysicsState(FArchive& Ar)
{
	const FUObjectMigrationContext* MigrationContext = Ar.GetMigrationContext();
	check(MigrationContext);

	if (Ar.IsLoading())
	{
		PVehicleOutput = MakeUnique<FPhysicsVehicleOutput>();
	}

	if (PVehicleOutput.IsValid())
	{
		FPhysicsVehicleOutput::StaticStruct()->SerializeItem(Ar, PVehicleOutput.Get(), nullptr);
	}

	if (Ar.IsLoading())
	{
		bIsVehicleStatesMigrated = true;
		MigratedVehicleStates.NetSerialize(Ar, nullptr, bIsVehicleStatesMigrated);
	}
	else if (Ar.IsSaving())
	{
		const EObjectMigrationSendType ObjMigrationSendType = MigrationContext->GetObjectMigrationSendType(this);

		if (CanSendMigratedPhysicsState(MigrationContext, ObjMigrationSendType))
		{
			bool bOutSuccess = true;
			if (bIsVehicleStatesMigrated)
			{
				UE_LOGF(LogVehicle, Verbose, "[%hs] Migrated a pending vehicle state", __func__);
				MigratedVehicleStates.NetSerialize(Ar, nullptr, bOutSuccess);
			}
			else
			{
				FNetworkVehicleStates VehicleStates;
				// BuildData accesses VehicleSimulationPT, which contains Physics Thread data.
				// While FPhysicsCommand is not inherently thread-safe due to PT ownership, 
				// this is currently safe because Dstm enforces synchronous physics execution.
				if (FPhysScene* PhysScene = GetWorld() ? GetWorld()->GetPhysicsScene() : nullptr)
				{
					FPhysicsCommand::ExecuteWrite(PhysScene, [this, &VehicleStates]()
						{
							VehicleStates.BuildData(this);
						});
				}
				else
				{
					UE_LOGF(LogVehicle, Warning, "[%hs] Cannot BuildData - no valid physics scene", __func__);
				}
				VehicleStates.NetSerialize(Ar, nullptr, bOutSuccess);
			}
		}
		else
		{
			UE_LOGF(LogVehicle, Log, "[%hs] Migrated an empty vehicle physics state", __func__);
			FNetworkVehicleStates VehicleStates;
			bool bOutSuccess = true;
			VehicleStates.NetSerialize(Ar, nullptr, bOutSuccess);
		}
	}
}

bool UChaosVehicleMovementComponent::CanSendMigratedPhysicsState(const FUObjectMigrationContext* MigrationContext, EObjectMigrationSendType ObjMigrationSendType) const
{
	if (ObjMigrationSendType == EObjectMigrationSendType::ReturnBorrowed)
	{
		return MigrationContext->MultiServerCommitRequestId.IsValid();
	}

	return true;
}

void UChaosVehicleMovementComponent::PostMigrate(const struct FUObjectMigrationContext& MigrationContext)
{
	Super::PostMigrate(MigrationContext);
}

void UChaosVehicleMovementComponent::ApplyMigratedVehicleState()
{
	if (bIsVehicleStatesMigrated)
	{
		MigratedVehicleStates.ApplyData(this);
		bIsVehicleStatesMigrated = false;
	}
}
#endif 

#if WITH_EDITOR
void UChaosVehicleMovementComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// Trigger a runtime rebuild of the Chaos vehicle
	FChaosVehicleManager::VehicleSetupTag++;

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void UChaosVehicleMovementComponent::SetUpdatedComponent(USceneComponent* NewUpdatedComponent)
{
	//Skip PawnMovementComponent and simply set PawnOwner to null if we don't have a PawnActor as owner
	UNavMovementComponent::SetUpdatedComponent(NewUpdatedComponent);
	PawnOwner = NewUpdatedComponent ? Cast<APawn>(NewUpdatedComponent->GetOwner()) : nullptr;

	if(USkeletalMeshComponent* SKC = Cast<USkeletalMeshComponent>(NewUpdatedComponent))
	{
		SKC->bLocalSpaceKinematics = true;
	}
}

void UChaosVehicleMovementComponent::SetOverrideController(AController* InOverrideController)
{
	OverrideController = InOverrideController;
}

bool UChaosVehicleMovementComponent::ShouldCreatePhysicsState() const
{
	if (!IsRegistered() || IsBeingDestroyed())
	{
		return false;
	}

	// only create 'Physics' vehicle in game
	UWorld* World = GetWorld();
	if (World->IsGameWorld())
	{
		FPhysScene* PhysScene = World->GetPhysicsScene();

		if (PhysScene && FChaosVehicleManager::GetVehicleManagerFromScene(PhysScene))
		{
			if (CanCreateVehicle())
			{
				return true;
			}
		}
	}

	return false;
}

bool UChaosVehicleMovementComponent::HasValidPhysicsState() const
{
	return PVehicleOutput.IsValid();
}

bool UChaosVehicleMovementComponent::CanCreateVehicle() const
{
	check(GetOwner());
	FString ActorName = GetOwner()->GetName();

	if (UpdatedComponent == NULL)
	{
		UE_LOGF(LogVehicle, Warning, "Can't create vehicle %ls (%ls). UpdatedComponent is not set.", *ActorName, *GetPathName());
		return false;
	}

	if (UpdatedPrimitive == NULL)
	{
		UE_LOGF(LogVehicle, Warning, "Can't create vehicle %ls (%ls). UpdatedComponent is not a PrimitiveComponent.", *ActorName, *GetPathName());
		return false;
	}

	return true;
}

void UChaosVehicleMovementComponent::OnCreatePhysicsState()
{
	Super::OnCreatePhysicsState();

	VehicleSetupTag = FChaosVehicleManager::VehicleSetupTag;

	// only create Physics vehicle in game
	UWorld* World = GetWorld();
	if (World->IsGameWorld())
	{
		if (FPhysScene* PhysScene = World->GetPhysicsScene())
		{
			if (FChaosVehicleManager::GetVehicleManagerFromScene(PhysScene))
			{
				CreateVehicle();
				FixupSkeletalMesh();

				if (PVehicleOutput)
				{
					FChaosVehicleManager* VehicleManager = FChaosVehicleManager::GetVehicleManagerFromScene(PhysScene);
					VehicleManager->AddVehicle(this);
				}
			}
			if (bUsingNetworkPhysicsPrediction)
			{
				if(NetworkPhysicsComponent)
				{
					NetworkPhysicsComponent->CreateDataHistory<FPhysicsVehicleTraits>(this);
				}
			}
		}
	}

	FBodyInstance* BodyInstance = nullptr;
	if (USkeletalMeshComponent* SkeletalMesh = GetSkeletalMesh())
	{	
		// this line was causing the server wheel positions to not be updated - this is already a user property so don't override it here and leave it to user to select the right option for their scenario
		//SkeletalMesh->VisibilityBasedAnimTickOption = EVisibilityBasedAnimTickOption::OnlyTickPoseWhenRendered;
		BodyInstance = &SkeletalMesh->BodyInstance;
	}
}

void UChaosVehicleMovementComponent::OnDestroyPhysicsState()
{
	Super::OnDestroyPhysicsState();

	if (PVehicleOutput.IsValid())
	{
		FChaosVehicleManager* VehicleManager = FChaosVehicleManager::GetVehicleManagerFromScene(GetWorld()->GetPhysicsScene());
		VehicleManager->RemoveVehicle(this);
		PVehicleOutput.Reset(nullptr);

		if (UpdatedComponent)
		{
			UpdatedComponent->RecreatePhysicsState();
		}
	}
	if (bUsingNetworkPhysicsPrediction && NetworkPhysicsComponent)
	{
		NetworkPhysicsComponent->RemoveDataHistory();
	}
}

void UChaosVehicleMovementComponent::PreTickGT(float DeltaTime)
{
	// movement updates and replication
	if (PVehicleOutput && UpdatedComponent)
	{
		APawn* MyOwner = Cast<APawn>(UpdatedComponent->GetOwner());
		if (MyOwner)
		{
			UpdateState(DeltaTime);
		}
	}

	if (!bUsingNetworkPhysicsPrediction)
	{
		FControlInputs ControlInputs;
		ControlInputs.ThrottleInput = ThrottleInput;
		ControlInputs.BrakeInput = BrakeInput;
		ControlInputs.SteeringInput = SteeringInput;
		ControlInputs.HandbrakeInput = HandbrakeInput;
		ControlInputs.RollInput = RollInput;
		ControlInputs.PitchInput = PitchInput;
		ControlInputs.YawInput = YawInput;
		ControlInputs.ParkingEnabled = bParkEnabled;
		ProcessSleeping(ControlInputs);
	}
	else
	{
		ProcessSleeping(VehicleSimulationPT->VehicleInputs);
	}

	if (VehicleSetupTag != FChaosVehicleManager::VehicleSetupTag)
	{
		RecreatePhysicsState();
	}
}

void UChaosVehicleMovementComponent::StopMovementImmediately()
{
	if (bUsingNetworkPhysicsPrediction)
	{
		return; // Called when player onwership changes I think - client restarts - resets velocity/forces bad for our case.
	}

	FBodyInstance* TargetInstance = GetBodyInstance();
	if (TargetInstance)
	{
		// if start awake is false then setting the velocity (even to zero) causes particle to wake up.
		if (TargetInstance->IsInstanceAwake())
		{
			TargetInstance->SetLinearVelocity(FVector::ZeroVector, false);
			TargetInstance->SetAngularVelocityInRadians(FVector::ZeroVector, false);
			TargetInstance->ClearForces();
			TargetInstance->ClearTorques();
		}
	}
	Super::StopMovementImmediately();
	ClearAllInput();
}

// Input

void UChaosVehicleMovementComponent::SetThrottleInput(float Throttle)
{
	RawThrottleInput = FMath::Clamp(Throttle, -1.0f, 1.0f);
}

void UChaosVehicleMovementComponent::IncreaseThrottleInput(float ThrottleDelta)
{
	RawThrottleInput = FMath::Clamp(RawThrottleInput + ThrottleDelta, 0.f, 1.0f);
}

void UChaosVehicleMovementComponent::DecreaseThrottleInput(float ThrottleDelta)
{
	RawThrottleInput = FMath::Clamp(RawThrottleInput - ThrottleDelta, 0.f, 1.0f);
}

void UChaosVehicleMovementComponent::SetBrakeInput(float Brake)
{
	RawBrakeInput = FMath::Clamp(Brake, -1.0f, 1.0f);
}

void UChaosVehicleMovementComponent::SetSteeringInput(float Steering)
{
	RawSteeringInput = FMath::Clamp(Steering, -1.0f, 1.0f);
}

void UChaosVehicleMovementComponent::SetPitchInput(float Pitch)
{
	RawPitchInput = FMath::Clamp(Pitch, -1.0f, 1.0f);
}

void UChaosVehicleMovementComponent::SetRollInput(float Roll)
{
	RawRollInput = FMath::Clamp(Roll, -1.0f, 1.0f);
}

void UChaosVehicleMovementComponent::SetYawInput(float Yaw)
{
	RawYawInput = FMath::Clamp(Yaw, -1.0f, 1.0f);
}

void UChaosVehicleMovementComponent::SetHandbrakeInput(bool bNewHandbrake)
{
	bRawHandbrakeInput = bNewHandbrake;
}


void UChaosVehicleMovementComponent::SetParked(bool bParked)
{
	bParkEnabled = bParked;
}

void UChaosVehicleMovementComponent::SetSleeping(bool bEnableSleep)
{
	if (bEnableSleep)
	{
		PutAllEnabledRigidBodiesToSleep();
		VehicleState.bSleeping = true;
	}
	else
	{
		WakeAllEnabledRigidBodies();
		VehicleState.bSleeping = false;
	}
}

void UChaosVehicleMovementComponent::SetChangeUpInput(bool bNewGearUp)
{
	bRawGearUpInput = bNewGearUp;
}

void UChaosVehicleMovementComponent::SetChangeDownInput(bool bNewGearDown)
{
	bRawGearDownInput = bNewGearDown;
}

void UChaosVehicleMovementComponent::SetTargetGear(int32 GearNum, bool bImmediate)
{
	if (GearNum != TargetGear)
	{
		if (bUsingNetworkPhysicsPrediction)
		{
			// With network physics prediction, the gear is set via the input data path
			// (Update -> ProcessInputs_Internal) to stay consistent with the input history.
			// Direct ExecuteWrite would conflict with ApplyData during resimulation.
		}
		else
		{
			// Without network physics prediction, set the gear directly on the PT.
			if (FBodyInstance* TargetInstance = GetBodyInstance())
			{
				FPhysicsCommand::ExecuteWrite(TargetInstance->GetPhysicsActor(), [&](const FPhysicsActorHandle& Chassis)
					{
						if (VehicleSimulationPT && VehicleSimulationPT->PVehicle && VehicleSimulationPT->PVehicle->HasTransmission())
						{
							VehicleSimulationPT->PVehicle->GetTransmission().SetGear(GearNum, bImmediate);
						}
					});
			}
		}

		TargetGear = GearNum;
	}
}

void UChaosVehicleMovementComponent::SetUseAutomaticGears(bool bUseAuto)
{
	TransmissionType = bUseAuto ? Chaos::ETransmissionType::Automatic : Chaos::ETransmissionType::Manual;
}

void UChaosVehicleMovementComponent::SetRequiresControllerForInputs(bool bRequiresController)
{
	bRequiresControllerForInputs = bRequiresController;
}

// Data access

int32 UChaosVehicleMovementComponent::GetCurrentGear() const
{
	return (PVehicleOutput) ? PVehicleOutput->CurrentGear : 0;
}

int32 UChaosVehicleMovementComponent::GetTargetGear() const
{
	return TargetGear;
}

bool UChaosVehicleMovementComponent::GetUseAutoGears() const
{
	return (TransmissionType == Chaos::ETransmissionType::Automatic);
}

float UChaosVehicleMovementComponent::GetForwardSpeed() const
{
	return VehicleState.ForwardSpeed;
}

float UChaosVehicleMovementComponent::GetForwardSpeedMPH() const
{
	return Chaos::CmSToMPH(GetForwardSpeed());
}

bool UChaosVehicleMovementComponent::IsParked() const
{
	return (bParkEnabled > 0);
}

// input related
float UChaosVehicleMovementComponent::CalcSteeringInput()
{
	return RawSteeringInput;
}

void UChaosVehicleMovementComponent::CalcThrottleBrakeInput(float& ThrottleOut, float& BrakeOut)
{
	BrakeOut = RawBrakeInput;
	ThrottleOut = RawThrottleInput;

	if (bReverseAsBrake)
	{
		if (RawThrottleInput > 0.f)
		{
			// car moving backwards but player wants to move forwards...
			// if vehicle is moving backwards, then press brake	
			if (bThrottleAsBrake && VehicleState.ForwardSpeed < -WrongDirectionThreshold)
			{
				BrakeOut = 1.0f;
				ThrottleOut = 0.0f;
			}

		}
		else if (RawBrakeInput > 0.f)
		{
			// car moving forwards but player wants to move backwards...
			// if vehicle is moving forwards, then press brake
			if (VehicleState.ForwardSpeed > WrongDirectionThreshold)
			{
				BrakeOut = 1.0f;
				ThrottleOut = 0.0f;
			}
			else if (GetTargetGear() < 0)
			{
				ThrottleOut = RawBrakeInput;
				BrakeOut = 0.0f;
			}
		}
		else
		{
			// if player isn't pressing forward or backwards...
			if (VehicleState.ForwardSpeed < StopThreshold && VehicleState.ForwardSpeed > -StopThreshold)	//auto brake 
			{
				// this code is causing the sleep state to toggle back off when the vehicle is stationary. It doesn't appear to be required anymore since the vehicle
				// has another mechainsim to prevent it rolling on hills, which I believe was the initial intention of the code.
				//BrakeOut = 1.f;
			}
			else
			{
				BrakeOut = IdleBrakeInput;
			}
		}

		ThrottleOut = FMath::Clamp<float>(ThrottleOut, 0.0, 1.0);
		BrakeOut = FMath::Clamp<float>(BrakeOut, 0.0, 1.0);
	}
	else
	{
		BrakeOut = FMath::Abs(RawBrakeInput);

		// if player isn't pressing forward or backwards...
		if (RawBrakeInput < SMALL_NUMBER && RawThrottleInput < SMALL_NUMBER)
		{
			if (VehicleState.ForwardSpeed < StopThreshold && VehicleState.ForwardSpeed > -StopThreshold)	//auto brake 
			{
				BrakeOut = 1.f;
			}
		}

	}

}

float UChaosVehicleMovementComponent::CalcHandbrakeInput()
{
	return (bRawHandbrakeInput == true) ? 1.0f : 0.0f;
}

float UChaosVehicleMovementComponent::CalcPitchInput()
{
	return RawPitchInput;
}

float UChaosVehicleMovementComponent::CalcRollInput()
{
	return RawRollInput;
}

float UChaosVehicleMovementComponent::CalcYawInput()
{
	return RawYawInput;
}

void UChaosVehicleMovementComponent::ClearInput()
{
	SteeringInput = 0.0f;
	ThrottleInput = 0.0f;
	BrakeInput = 0.0f;
	HandbrakeInput = 0.0f;
	PitchInput = 0.0f;
	RollInput = 0.0f;
	YawInput = 0.0f;

	// Send this immediately.
	int32 CurrentGear = 0;
	if (PVehicleOutput)
	{
		CurrentGear = PVehicleOutput->CurrentGear;
	}

	AController* Controller = GetController();
	if (Controller && Controller->IsLocalController() && PVehicleOutput)
	{
		if (!bUsingNetworkPhysicsPrediction)
		{
			ServerUpdateState(SteeringInput, ThrottleInput, BrakeInput, HandbrakeInput, CurrentGear, RollInput, PitchInput, YawInput);
		}
	}
}

void UChaosVehicleMovementComponent::ClearRawInput()
{
	RawBrakeInput = 0.0f;
	RawSteeringInput = 0.0f;
	RawThrottleInput = 0.0f;
	RawPitchInput = 0.0f;
	RawRollInput = 0.0f;
	RawYawInput = 0.0f;
	bRawGearDownInput = false;
	bRawGearUpInput = false;
	bRawHandbrakeInput = false;
}

// Update

void UChaosVehicleMovementComponent::UpdateState(float DeltaTime)
{
	// update input values
	AController* Controller = GetController();
	VehicleState.CaptureState(GetBodyInstance(), GetGravityZ(), DeltaTime);
	VehicleState.NumWheelsOnGround = 0;
	VehicleState.bVehicleInAir = false;
	int NumWheels = 0;
	if (PVehicleOutput)
	{
		for (int WheelIdx = 0; WheelIdx < PVehicleOutput->Wheels.Num(); WheelIdx++)
		{
			if (PVehicleOutput->Wheels[WheelIdx].InContact)
			{
				VehicleState.NumWheelsOnGround++;
			}
			else
			{
				VehicleState.bVehicleInAir = true;
			}
			NumWheels++;
		}
	}
	VehicleState.bAllWheelsOnGround = (VehicleState.NumWheelsOnGround == NumWheels);

	bool bProcessLocally = bRequiresControllerForInputs ? (Controller && Controller->IsLocalController()) : true;

	// IsLocallyControlled will fail if the owner is unpossessed (i.e. Controller == nullptr);
	// Should we remove input instead of relying on replicated state in that case?
	if (bProcessLocally && PVehicleOutput)
	{
		// For automatic transmission with network physics prediction, the physics simulation
		// (TransmissionSystem::Simulate) owns gear changes. Sync the GT's TargetGear from the
		// physics output so FillAsyncInput doesn't send a stale value that overrides the
		// simulation's automatic upshift/downshift decisions each frame.
		// See UE-374761: gear stuck at N because GT kept forcing TargetGear=1 via ProcessInputs_Internal.
		if (bUsingNetworkPhysicsPrediction && TransmissionType == Chaos::ETransmissionType::Automatic)
		{
			TargetGear = PVehicleOutput->TargetGear;
		}

		if (bReverseAsBrake)
		{
			// Automatically shift between reverse and first gear based on input.
			// Throttle takes priority when both inputs are pressed to avoid gear oscillation.
			if (RawThrottleInput > KINDA_SMALL_NUMBER && GetTargetGear() <= 0)
			{
				SetTargetGear(1, true);
			}
			else if (RawBrakeInput > KINDA_SMALL_NUMBER && RawThrottleInput <= KINDA_SMALL_NUMBER && GetTargetGear() >= 0)
			{
				if (FMath::Abs(GetForwardSpeed()) < WrongDirectionThreshold)
				{
					SetTargetGear(-1, true);
				}
			}
		}
		else
		{
			if (TransmissionType == Chaos::ETransmissionType::Automatic)
			{
				if (RawThrottleInput > KINDA_SMALL_NUMBER
					&& GetCurrentGear() == 0
					&& GetTargetGear() == 0)
				{
					SetTargetGear(1, true);
				}
			}

		}

		float ModifiedThrottle = 0.f;
		float ModifiedBrake = 0.f;
		CalcThrottleBrakeInput(ModifiedThrottle, ModifiedBrake);

		// Apply Inputs locally
		SteeringInput = SteeringInputRate.InterpInputValue(DeltaTime, SteeringInput, CalcSteeringInput());
		ThrottleInput = ThrottleInputRate.InterpInputValue(DeltaTime, ThrottleInput, ModifiedThrottle);
		BrakeInput = BrakeInputRate.InterpInputValue(DeltaTime, BrakeInput, ModifiedBrake);
		PitchInput = PitchInputRate.InterpInputValue(DeltaTime, PitchInput, CalcPitchInput());
		RollInput = RollInputRate.InterpInputValue(DeltaTime, RollInput, CalcRollInput());
		YawInput = YawInputRate.InterpInputValue(DeltaTime, YawInput, CalcYawInput());
		HandbrakeInput = HandbrakeInputRate.InterpInputValue(DeltaTime, HandbrakeInput, CalcHandbrakeInput());

		if (!bUsingNetworkPhysicsPrediction)
		{
			// and send to server - (ServerUpdateState_Implementation below)
			ServerUpdateState(SteeringInput, ThrottleInput, BrakeInput, HandbrakeInput, GetCurrentGear(), RollInput, PitchInput, YawInput);
		}

		if (PawnOwner && PawnOwner->IsNetMode(NM_Client))
		{
			MarkForClientCameraUpdate();
		}
	}
	else if (!bUsingNetworkPhysicsPrediction)
	{
		// use replicated values for remote pawns
		SteeringInput = ReplicatedState.SteeringInput;
		ThrottleInput = ReplicatedState.ThrottleInput;
		BrakeInput = ReplicatedState.BrakeInput;
		PitchInput = ReplicatedState.PitchInput;
		RollInput = ReplicatedState.RollInput;
		YawInput = ReplicatedState.YawInput;
		HandbrakeInput = ReplicatedState.HandbrakeInput;
		SetTargetGear(ReplicatedState.TargetGear, true);
	}
}


void UChaosVehicleMovementComponent::ProcessSleeping(const FControlInputs& ControlInputs)
{
	FBodyInstance* TargetInstance = GetBodyInstance();
	if (TargetInstance)
	{
		bool PrevSleeping = VehicleState.bSleeping;
		VehicleState.bSleeping = !TargetInstance->IsInstanceAwake();

		// The physics system has woken vehicle up due to a collision or something
		if (PrevSleeping && !VehicleState.bSleeping)
		{
			VehicleState.SleepCounter = 0;
		}

		// If the vehicle is locally controlled, we want to use the raw inputs to determine sleep.
		// However, if it's on the Server or is just being replicated to other Clients then there
		// won't be any Raw input. In that case, use ReplicatedState instead.

		// NOTE: Even on local clients, ReplicatedState will still be populated (the call to ServerUpdateState will
		//			be processed locally). Maybe we should *just* use ReplicatedState?

		const AController* Controller = GetController();
		const bool bIsLocallyControlled = (Controller && Controller->IsLocalController()) || bUsingNetworkPhysicsPrediction;
		const bool bControlInputPressed = bIsLocallyControlled ? (ControlInputs.ThrottleInput >= GVehicleDebugParams.ControlInputWakeTolerance)
			|| (ControlInputs.BrakeInput >= GVehicleDebugParams.ControlInputWakeTolerance)
			|| (FMath::Abs(ControlInputs.SteeringInput - PrevSteeringInput) >= GVehicleDebugParams.ControlInputWakeTolerance)
			|| (ControlInputs.RollInput >= GVehicleDebugParams.ControlInputWakeTolerance)
			|| (ControlInputs.PitchInput >= GVehicleDebugParams.ControlInputWakeTolerance)
			|| (ControlInputs.YawInput >= GVehicleDebugParams.ControlInputWakeTolerance)
			: (ReplicatedState.ThrottleInput >= GVehicleDebugParams.ControlInputWakeTolerance)
			|| (ReplicatedState.BrakeInput >= GVehicleDebugParams.ControlInputWakeTolerance)
			|| (FMath::Abs(ReplicatedState.SteeringInput - PrevReplicatedSteeringInput) >= GVehicleDebugParams.ControlInputWakeTolerance)
			|| (ReplicatedState.RollInput >= GVehicleDebugParams.ControlInputWakeTolerance)
			|| (ReplicatedState.PitchInput >= GVehicleDebugParams.ControlInputWakeTolerance)
			|| (ReplicatedState.YawInput >= GVehicleDebugParams.ControlInputWakeTolerance);

		PrevSteeringInput = ControlInputs.SteeringInput;
		PrevReplicatedSteeringInput = ReplicatedState.SteeringInput;

		// Wake if control input pressed
		if (bControlInputPressed || GVehicleDebugParams.DisableVehicleSleep)
		{
			VehicleState.bSleeping = false;
			VehicleState.SleepCounter = 0;
			SetSleeping(false);
		}
		else if (!GVehicleDebugParams.DisableVehicleSleep && !VehicleState.bSleeping && !bControlInputPressed && VehicleState.bAllWheelsOnGround && (VehicleState.VehicleUpAxis.Z > SleepSlopeLimit))
		{
			float SpeedSqr = TargetInstance->GetUnrealWorldVelocity().SizeSquared();
			if (SpeedSqr < (SleepThreshold * SleepThreshold))
			{
				if (VehicleState.SleepCounter < GVehicleDebugParams.SleepCounterThreshold)
				{
					VehicleState.SleepCounter++;
				}
				else
				{
					VehicleState.bSleeping = true;
					SetSleeping(true);
				}
			}
		}
	}
}

/// @cond DOXYGEN_WARNINGS

bool UChaosVehicleMovementComponent::ServerUpdateState_Validate(float InSteeringInput, float InThrottleInput, float InBrakeInput, float InHandbrakeInput, int32 InCurrentGear, float InRollInput, float InPitchInput, float InYawInput)
{
	return true;
}

void UChaosVehicleMovementComponent::ServerUpdateState_Implementation(float InSteeringInput, float InThrottleInput, float InBrakeInput
	, float InHandbrakeInput, int32 InCurrentGear, float InRollInput, float InPitchInput, float InYawInput)
{
	SteeringInput = InSteeringInput;
	ThrottleInput = InThrottleInput;
	BrakeInput = InBrakeInput;
	HandbrakeInput = InHandbrakeInput;
	RollInput = InRollInput;
	PitchInput = InPitchInput;
	YawInput = InYawInput;

	if (!GetUseAutoGears())
	{
		if (UWorld* World = GetWorld())
		{
			if (World->GetNetMode() == NM_DedicatedServer)
			{
				SetTargetGear(InCurrentGear, true);
			}
		}
	}

	// update state of inputs
	ReplicatedState.SteeringInput = InSteeringInput;
	ReplicatedState.ThrottleInput = InThrottleInput;
	ReplicatedState.BrakeInput = InBrakeInput;
	ReplicatedState.HandbrakeInput = InHandbrakeInput;
	ReplicatedState.TargetGear = InCurrentGear;
	ReplicatedState.RollInput = InRollInput;
	ReplicatedState.PitchInput = InPitchInput;
	ReplicatedState.YawInput = InYawInput;

}

/// @endcond


// Setup
AController* UChaosVehicleMovementComponent::GetController() const
{
	if (OverrideController)
	{
		return OverrideController;
	}

	if (UpdatedComponent)
	{
		if (APawn* Pawn = Cast<APawn>(UpdatedComponent->GetOwner()))
		{
			return Pawn->GetController();
		}
	}

	return nullptr;
}

APlayerController* UChaosVehicleMovementComponent::GetPlayerController() const
{
	if (AController* Controller = GetController())
	{
		return Cast<APlayerController>(Controller);
	}
	return nullptr;
}


FBodyInstance* UChaosVehicleMovementComponent::GetBodyInstance()
{
	return UpdatedPrimitive ? UpdatedPrimitive->GetBodyInstance() : nullptr;
}

const FBodyInstance* UChaosVehicleMovementComponent::GetBodyInstance() const
{
	return UpdatedPrimitive ? UpdatedPrimitive->GetBodyInstance() : nullptr;
}

UMeshComponent* UChaosVehicleMovementComponent::GetMesh() const
{
	return Cast<UMeshComponent>(UpdatedComponent);
}

USkeletalMeshComponent* UChaosVehicleMovementComponent::GetSkeletalMesh()
{
	return Cast<USkeletalMeshComponent>(UpdatedComponent);
}

UStaticMeshComponent* UChaosVehicleMovementComponent::GetStaticMesh()
{
	return Cast<UStaticMeshComponent>(UpdatedComponent);
}

FVector UChaosVehicleMovementComponent::LocateBoneOffset(const FName InBoneName, const FVector& InExtraOffset) const
{
	FVector Offset = InExtraOffset;

	if (InBoneName != NAME_None)
	{
		if (USkinnedMeshComponent* Mesh = Cast<USkinnedMeshComponent>(GetMesh()))
		{
			if (ensureMsgf(Mesh->GetSkinnedAsset(), TEXT("Expected skinned asset when locating bone offset. Asset might be missing references.")))
			{
				const FVector BonePosition = Mesh->GetSkinnedAsset()->GetComposedRefPoseMatrix(InBoneName).GetOrigin() * Mesh->GetRelativeScale3D();
				//BonePosition is local for the root BONE of the skeletal mesh - however, we are using the Root BODY which may have its own transform, so we need to return the position local to the root BODY
				FMatrix RootBodyMTX = FMatrix::Identity;

				if (Mesh->GetBodyInstance() && Mesh->GetBodyInstance()->BodySetup.IsValid())
				{
					RootBodyMTX = Mesh->GetSkinnedAsset()->GetComposedRefPoseMatrix(Mesh->GetBodyInstance()->BodySetup->BoneName);
				}
				const FVector LocalBonePosition = RootBodyMTX.InverseTransformPosition(BonePosition);
				Offset += LocalBonePosition;
			}
		}
	}
	return Offset;
}

void UChaosVehicleMovementComponent::CreateVehicle()
{
	ComputeConstants();

	{
		if (CanCreateVehicle())
		{
			check(UpdatedComponent);
			if (ensure(UpdatedPrimitive != nullptr))
			{
				TUniquePtr<Chaos::FSimpleWheeledVehicle> PVehicle = CreatePhysicsVehicle();

				// Low level physics representation
				SetupVehicle(PVehicle); 

				if (PVehicleOutput != nullptr)
				{
					PostSetupVehicle();
				}

				// Physics thread simulation class will now take ownership of the PVehicle pointer, we cannot safely use it anymore from the game thread
				VehicleSimulationPT->Init(PVehicle);

			}

			VehicleState.CaptureState(GetBodyInstance(), GetGravityZ(), 0.01667f);

		}
	}
}

void UChaosVehicleMovementComponent::SetupVehicle(TUniquePtr<Chaos::FSimpleWheeledVehicle>& PVehicle)
{
	Chaos::FSimpleAerodynamicsSim AerodynamicsSim(&GetAerodynamicsConfig());
	PVehicle->Aerodynamics.Add(AerodynamicsSim);

	for (FVehicleAerofoilConfig& AerofoilSetup : Aerofoils)
	{
		Chaos::FAerofoil AerofoilSim(&AerofoilSetup.GetPhysicsAerofoilConfig(*this));
		PVehicle->Aerofoils.Add(AerofoilSim);
	}

	for (FVehicleThrustConfig& ThrustSetup : Thrusters)
	{
		Chaos::FSimpleThrustSim ThrustSim(&ThrustSetup.GetPhysicsThrusterConfig(*this));
		PVehicle->Thrusters.Add(ThrustSim);
	}
}

void UChaosVehicleMovementComponent::PostSetupVehicle()
{
}

void UChaosVehicleMovementComponent::SetupVehicleMass()
{
	if (UpdatedPrimitive && UpdatedPrimitive->GetBodyInstance())
	{
		//Ensure that if mass properties ever change we set them back to our override
		UpdatedPrimitive->GetBodyInstance()->OnRecalculatedMassProperties().AddUObject(this, &UChaosVehicleMovementComponent::UpdateMassProperties);
#if UE_WITH_REMOTE_OBJECT_HANDLE
		// Skip inertia scaling when vehicle is migrated since its scaled inertia is already migrated. 
		if (!bIsMigrated)
#endif
		{
			UpdateMassProperties(UpdatedPrimitive->GetBodyInstance());
		}
	}
}

void UChaosVehicleMovementComponent::UpdateMassProperties(FBodyInstance* BodyInstance)
{
	if (BodyInstance && FPhysicsInterface::IsValid(BodyInstance->GetPhysicsActor()) && FPhysicsInterface::IsRigidBody(BodyInstance->GetPhysicsActor()))
	{
		FPhysicsCommand::ExecuteWrite(BodyInstance->GetPhysicsActor(), [&](const FPhysicsActorHandle& Actor)
			{
				const float MassRatio = this->Mass > 0.0f ? this->Mass / BodyInstance->GetBodyMass() : 1.0f;

				FVector InertiaTensor = BodyInstance->GetBodyInertiaTensor();

				InertiaTensor.X *= this->InertiaTensorScale.X * MassRatio;
				InertiaTensor.Y *= this->InertiaTensorScale.Y * MassRatio;
				InertiaTensor.Z *= this->InertiaTensorScale.Z * MassRatio;

				if (bEnableCenterOfMassOverride)
				{
					FTransform COMTransform = FPhysicsInterface::GetComTransformLocal_AssumesLocked(Actor);
					COMTransform.SetTranslation(CenterOfMassOverride + BodyInstance->COMNudge);
					FPhysicsInterface::SetComLocalPose_AssumesLocked(Actor, COMTransform);
				}
				FPhysicsInterface::SetMassSpaceInertiaTensor_AssumesLocked(Actor, InertiaTensor);
				FPhysicsInterface::SetMass_AssumesLocked(Actor, this->Mass);
			});
	}

}

void UChaosVehicleMovementComponent::ComputeConstants()
{
	DragArea = ChassisWidth * ChassisHeight;
}


// Debug
void UChaosVehicleMovementComponent::ShowDebugInfo(AHUD* HUD, UCanvas* Canvas, const FDebugDisplayInfo& DisplayInfo, float& YL, float& YPos)
{
	static FName NAME_Vehicle = FName(TEXT("Vehicle"));

	if (Canvas && HUD->ShouldDisplayDebug(NAME_Vehicle))
	{
		if (APlayerController* Controller = Cast<APlayerController>(GetController()))
		{
			if (Controller->IsLocalController())
			{
				DrawDebug(Canvas, YL, YPos);
			}
		}
	}
}

void UChaosVehicleMovementComponent::DrawDebug(UCanvas* Canvas, float& YL, float& YPos)
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	FBodyInstance* TargetInstance = GetBodyInstance();
	if (PVehicleOutput == nullptr || TargetInstance == nullptr)
	{
		return;
	}

	UFont* RenderFont = GEngine->GetMediumFont();
	// draw general vehicle data
	{
		Canvas->SetDrawColor(FColor::White);
		YPos += 16;

		float ForwardSpeedKmH = Chaos::CmSToKmH(GetForwardSpeed());
		float ForwardSpeedMPH = Chaos::CmSToMPH(GetForwardSpeed());
		float ForwardSpeedMSec = Chaos::CmToM(GetForwardSpeed());

		if (TargetInstance)
		{
			FVector FinalCOM = TargetInstance->GetMassSpaceLocal().GetTranslation();
			FVector OffsetCOM = TargetInstance->COMNudge;
			FVector BaseCOM = FinalCOM - TargetInstance->COMNudge;
			YPos += Canvas->DrawText(RenderFont, FString::Printf(TEXT("Mass (Kg): %.1f"), TargetInstance->GetBodyMass()), 4, YPos);
			YPos += Canvas->DrawText(RenderFont, FString::Printf(TEXT("Local COM : %s"), *FinalCOM.ToString()), 4, YPos);
			YPos += Canvas->DrawText(RenderFont, FString::Printf(TEXT("[COM Base : %s  COM Offset : %s]"), *BaseCOM.ToString(), *OffsetCOM.ToString()), 4, YPos);
			YPos += Canvas->DrawText(RenderFont, FString::Printf(TEXT("Inertia : %s"), *TargetInstance->GetBodyInertiaTensor().ToString()), 4, YPos);
		}

		YPos += Canvas->DrawText(RenderFont, FString::Printf(TEXT("Awake %d (Vehicle Sleep %d)"), TargetInstance->IsInstanceAwake(), VehicleState.bSleeping), 4, YPos);
		YPos += Canvas->DrawText(RenderFont, FString::Printf(TEXT("Speed (km/h): %.1f  (MPH): %.1f  (m/s): %.1f"), ForwardSpeedKmH, ForwardSpeedMPH, ForwardSpeedMSec), 4, YPos);
		YPos += Canvas->DrawText(RenderFont, FString::Printf(TEXT("Acceleration (m/s-2): %.1f"), Chaos::CmToM(VehicleState.LocalAcceleration.X)), 4, YPos);
		YPos += Canvas->DrawText(RenderFont, FString::Printf(TEXT("GForce : %2.1f"), VehicleState.LocalGForce.X), 4, YPos);
		YPos += Canvas->DrawText(RenderFont, FString::Printf(TEXT("Steering: %.1f (RAW %.1f)"), SteeringInput, RawSteeringInput), 4, YPos);
		YPos += Canvas->DrawText(RenderFont, FString::Printf(TEXT("Throttle: %.1f (RAW %.1f)"), ThrottleInput, RawThrottleInput), 4, YPos);
		YPos += Canvas->DrawText(RenderFont, FString::Printf(TEXT("Brake: %.1f (RAW %.1f)"), BrakeInput, RawBrakeInput), 4, YPos);
		YPos += Canvas->DrawText(RenderFont, FString::Printf(TEXT("Roll: %.1f (RAW %.1f)"), RollInput, RawRollInput), 4, YPos);
		YPos += Canvas->DrawText(RenderFont, FString::Printf(TEXT("Pitch: %.1f (RAW %.1f)"), PitchInput, RawPitchInput), 4, YPos);
		YPos += Canvas->DrawText(RenderFont, FString::Printf(TEXT("Yaw: %.1f (RAW %.1f)"), YawInput, RawYawInput), 4, YPos);
		FString GearState = GetUseAutoGears() ? "Automatic" : "Manual";
		YPos += Canvas->DrawText(RenderFont, FString::Printf(TEXT("Gears: %s"), *GearState), 4, YPos);
	}

#endif
}

/// @cond DOXYGEN_WARNINGS

void UChaosVehicleMovementComponent::GetLifetimeReplicatedProps(TArray< FLifetimeProperty >& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	DOREPLIFETIME(UChaosVehicleMovementComponent, ReplicatedState);
	DOREPLIFETIME(UChaosVehicleMovementComponent, OverrideController);
}

/// @endcond


void UChaosVehicleMovementComponent::DrawLine2D(UCanvas* Canvas, const FVector2D& StartPos, const FVector2D& EndPos, FColor Color, float Thickness)
{
	if (Canvas)
	{
		FCanvasLineItem LineItem(StartPos, EndPos);
		LineItem.SetColor(Color);
		LineItem.LineThickness = Thickness;
		Canvas->DrawItem(LineItem);
	}
}


TUniquePtr<Chaos::FSimpleWheeledVehicle> UChaosVehicleMovementComponent::CreatePhysicsVehicle()
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	// Don't reinitialize PVehicleOutput when migrated because its data is already migrated over. 
	if (!bIsMigrated)
#endif
	{
		PVehicleOutput = MakeUnique<FPhysicsVehicleOutput>();	// create physics output container
	}
	return MakeUnique<Chaos::FSimpleWheeledVehicle>();		// create physics sim
}

void FVehicleAerofoilConfig::FillAerofoilSetup(const UChaosVehicleMovementComponent& MovementComponent)
{
	PAerofoilConfig.Type = (Chaos::EAerofoilType)(this->AerofoilType);
	PAerofoilConfig.Offset = MovementComponent.LocateBoneOffset(this->BoneName, this->Offset);
	PAerofoilConfig.UpAxis = this->UpAxis;
	PAerofoilConfig.Area = this->Area;
	PAerofoilConfig.Camber = this->Camber;
	PAerofoilConfig.MaxControlAngle = this->MaxControlAngle;
	PAerofoilConfig.StallAngle = this->StallAngle;
	PAerofoilConfig.LiftMultiplier = this->LiftMultiplier;
	PAerofoilConfig.DragMultiplier = this->DragMultiplier;
}

void FVehicleThrustConfig::FillThrusterSetup(const UChaosVehicleMovementComponent& MovementComponent)
{
	PThrusterConfig.Type = (Chaos::EThrustType)(this->ThrustType);
	PThrusterConfig.Offset = MovementComponent.LocateBoneOffset(this->BoneName, this->Offset);
	PThrusterConfig.Axis = this->ThrustAxis;
	//	PThrusterConfig.ThrustCurve = this->ThrustCurve;
	PThrusterConfig.MaxThrustForce = Chaos::MToCm(this->MaxThrustForce);
	PThrusterConfig.MaxControlAngle = this->MaxControlAngle;
}


TUniquePtr<FChaosVehicleAsyncInput> UChaosVehicleMovementComponent::SetCurrentAsyncInputOutput(int32 InputIdx, FChaosVehicleManagerAsyncOutput* CurOutput, FChaosVehicleManagerAsyncOutput* NextOutput, float Alpha, int32 VehicleManagerTimestamp)
{
	TUniquePtr<FChaosVehicleAsyncInput> CurInput = MakeUnique<FChaosVehicleAsyncInput>();
	SetCurrentAsyncInputOutputInternal(CurInput.Get(), InputIdx, CurOutput, NextOutput, Alpha, VehicleManagerTimestamp);
	return CurInput;
}


/************************************************************************/
/* Setup the current async I/O data                                     */
/************************************************************************/
void UChaosVehicleMovementComponent::SetCurrentAsyncInputOutputInternal(FChaosVehicleAsyncInput* CurInput, int32 InputIdx, FChaosVehicleManagerAsyncOutput* CurOutput, FChaosVehicleManagerAsyncOutput* NextOutput, float Alpha, int32 VehicleManagerTimestamp)
{
	ensure(CurAsyncInput == nullptr);	//should be reset after it was filled
	ensure(CurAsyncOutput == nullptr);	//should get reset after update is done

	CurAsyncInput = CurInput;
	CurAsyncInput->Vehicle = this;
	CurAsyncType = CurInput->Type;
	NextAsyncOutput = nullptr;
	OutputInterpAlpha = 0.f;

	// We need to find our vehicle in the output given
	if (CurOutput)
	{
		for (int32 PendingOutputIdx = 0; PendingOutputIdx < OutputsWaitingOn.Num(); ++PendingOutputIdx)
		{
			// Found the correct pending output, use index to get the vehicle.
			if (OutputsWaitingOn[PendingOutputIdx].Timestamp == CurOutput->Timestamp)
			{
				const int32 VehicleIdx = OutputsWaitingOn[PendingOutputIdx].Idx;
				FChaosVehicleAsyncOutput* VehicleOutput = CurOutput->VehicleOutputs[VehicleIdx].Get();
				if (VehicleOutput && VehicleOutput->bValid && VehicleOutput->Type == CurAsyncType)
				{
					CurAsyncOutput = VehicleOutput;

					if (NextOutput && NextOutput->Timestamp == CurOutput->Timestamp)
					{
						// This can occur when substepping - in this case, VehicleOutputs will be in the same order in NextOutput and CurOutput.
						FChaosVehicleAsyncOutput* VehicleNextOutput = NextOutput->VehicleOutputs[VehicleIdx].Get();
						if (VehicleNextOutput && VehicleNextOutput->bValid && VehicleNextOutput->Type == CurAsyncType)
						{
							NextAsyncOutput = VehicleNextOutput;
							OutputInterpAlpha = Alpha;
						}
					}
				}

				// these are sorted by timestamp, we are using latest, so remove entries that came before it.
				TArray<FAsyncOutputWrapper> NewOutputsWaitingOn;
				for (int32 CopyIndex = PendingOutputIdx; CopyIndex < OutputsWaitingOn.Num(); ++CopyIndex)
				{
					NewOutputsWaitingOn.Add(OutputsWaitingOn[CopyIndex]);
				}

				OutputsWaitingOn = MoveTemp(NewOutputsWaitingOn);
				break;
			}
		}

	}

	if (NextOutput && CurOutput)
	{
		if (NextOutput->Timestamp != CurOutput->Timestamp)
		{
			// NextOutput and CurOutput occurred in different steps, so we need to search for our specific vehicle.
			for (int32 PendingOutputIdx = 0; PendingOutputIdx < OutputsWaitingOn.Num(); ++PendingOutputIdx)
			{
				// Found the correct pending output, use index to get the vehicle.
				if (OutputsWaitingOn[PendingOutputIdx].Timestamp == NextOutput->Timestamp)
				{
					FChaosVehicleAsyncOutput* VehicleOutput = NextOutput->VehicleOutputs[OutputsWaitingOn[PendingOutputIdx].Idx].Get();
					if (VehicleOutput && VehicleOutput->bValid && VehicleOutput->Type == CurAsyncType)
					{
						NextAsyncOutput = VehicleOutput;
						OutputInterpAlpha = Alpha;
					}
					break;
				}
			}
		}
	}

	FAsyncOutputWrapper& NewOutput = OutputsWaitingOn.AddDefaulted_GetRef();
	NewOutput.Timestamp = VehicleManagerTimestamp;
	NewOutput.Idx = InputIdx;
}


// ---- ASYNC ----

/************************************************************************/
/* PASS ANY INUTS TO THE PHYSICS THREAD SIMULATION IN HERE              */
/************************************************************************/
void UChaosVehicleMovementComponent::Update(float DeltaTime)
{
	if (CurAsyncInput)
	{
		if (const FBodyInstance* BodyInstance = GetBodyInstance())
		{
			if (auto Handle = BodyInstance->GetPhysicsActor())
			{
				CurAsyncInput->Proxy = Handle;	// vehicles are never static
				FChaosVehicleAsyncInput* AsyncInput = static_cast<FChaosVehicleAsyncInput*>(CurAsyncInput);

				AsyncInput->PhysicsInputs.NetworkInputs.VehicleInputs.ThrottleInput = ThrottleInputRate.CalcControlFunction(ThrottleInput);
				AsyncInput->PhysicsInputs.NetworkInputs.VehicleInputs.BrakeInput = BrakeInputRate.CalcControlFunction(BrakeInput);
				AsyncInput->PhysicsInputs.NetworkInputs.VehicleInputs.SteeringInput = SteeringInputRate.CalcControlFunction(SteeringInput);
				AsyncInput->PhysicsInputs.NetworkInputs.VehicleInputs.HandbrakeInput = HandbrakeInput;
				AsyncInput->PhysicsInputs.NetworkInputs.VehicleInputs.RollInput = RollInputRate.CalcControlFunction(RollInput);
				AsyncInput->PhysicsInputs.NetworkInputs.VehicleInputs.PitchInput = PitchInputRate.CalcControlFunction(PitchInput);
				AsyncInput->PhysicsInputs.NetworkInputs.VehicleInputs.YawInput = YawInputRate.CalcControlFunction(YawInput);
				AsyncInput->PhysicsInputs.NetworkInputs.VehicleInputs.GearUpInput = bRawGearUpInput;
				AsyncInput->PhysicsInputs.NetworkInputs.VehicleInputs.GearDownInput = bRawGearDownInput;
				AsyncInput->PhysicsInputs.NetworkInputs.VehicleInputs.TransmissionType = TransmissionType;
				AsyncInput->PhysicsInputs.NetworkInputs.VehicleInputs.ParkingEnabled = bParkEnabled;

				// Pass the GT's gear decision through the input data path so it flows into
				// the network physics input history via BuildData/ApplyData consistently
				AsyncInput->PhysicsInputs.NetworkInputs.TransmissionTargetGear = TargetGear;
				AsyncInput->PhysicsInputs.NetworkInputs.TransmissionCurrentGear = TargetGear;

				// debug feature to limit the vehicles top speed
				if ((GVehicleDebugParams.SetMaxMPH > 0.f) && (FMath::Abs(ThrottleInput) > 0.0f) && FMath::Abs(GetForwardSpeedMPH()) >= GVehicleDebugParams.SetMaxMPH)
				{
					AsyncInput->PhysicsInputs.NetworkInputs.VehicleInputs.ThrottleInput = 0.1f;
				}

				AsyncInput->PhysicsInputs.GravityZ = GetGravityZ();
			}
		}
	}
}

void UChaosVehicleMovementComponent::ResetVehicleState()
{
	ClearRawInput();
	StopMovementImmediately();

	OnDestroyPhysicsState();
	OnCreatePhysicsState();

	// Shift into neutral, force the local copy of target gear to be correct
	SetTargetGear(0, true);
	TargetGear = 0;
}

void UChaosVehicleMovementComponent::FinalizeSimCallbackData(FChaosVehicleManagerAsyncInput& Input)
{
	CurAsyncInput = nullptr;
	CurAsyncOutput = nullptr;
}


/***************************************************************************/
/* READ OUTPUT DATA - Access the async output data from the Physics Thread */
/***************************************************************************/
void UChaosVehicleMovementComponent::ParallelUpdate(float DeltaSeconds)
{
	if (const FChaosVehicleAsyncOutput* CurrentOutput = static_cast<FChaosVehicleAsyncOutput*>(CurAsyncOutput))
	{
		if (CurrentOutput->bValid && PVehicleOutput)
		{
			// TODO: It would be nicer to go through CurAsyncOutput rather
			// than copying into the vehicle, think about non-async path
			PVehicleOutput->CurrentGear = CurAsyncOutput->VehicleSimOutput.CurrentGear;
			PVehicleOutput->TargetGear = CurAsyncOutput->VehicleSimOutput.TargetGear;

			// WHEN RUNNING WITH ASYNC ON & FIXED TIMESTEP THEN WE NEED TO INTERPOLATE BETWEEN THE CURRENT AND NEXT OUTPUT RESULTS
			if (const FChaosVehicleAsyncOutput* NextOutput = static_cast<FChaosVehicleAsyncOutput*>(NextAsyncOutput))
			{
				PVehicleOutput->EngineRPM = FMath::Lerp(CurAsyncOutput->VehicleSimOutput.EngineRPM, NextAsyncOutput->VehicleSimOutput.EngineRPM, OutputInterpAlpha);
				PVehicleOutput->EngineTorque = FMath::Lerp(CurAsyncOutput->VehicleSimOutput.EngineTorque, NextAsyncOutput->VehicleSimOutput.EngineTorque, OutputInterpAlpha);
				PVehicleOutput->TransmissionRPM = FMath::Lerp(CurAsyncOutput->VehicleSimOutput.TransmissionRPM, NextAsyncOutput->VehicleSimOutput.TransmissionRPM, OutputInterpAlpha);
				PVehicleOutput->TransmissionTorque = FMath::Lerp(CurAsyncOutput->VehicleSimOutput.TransmissionTorque, NextAsyncOutput->VehicleSimOutput.TransmissionTorque, OutputInterpAlpha);

				for (int WheelIdx = 0; WheelIdx < CurrentOutput->VehicleSimOutput.Wheels.Num(); WheelIdx++)
				{
					const FWheelsOutput& Current = CurrentOutput->VehicleSimOutput.Wheels[WheelIdx];
					const FWheelsOutput& Next = NextOutput->VehicleSimOutput.Wheels[WheelIdx];

					PVehicleOutput->Wheels[WheelIdx].InContact = Current.InContact;
					PVehicleOutput->Wheels[WheelIdx].SteeringAngle = FMath::Lerp(Current.SteeringAngle, Next.SteeringAngle, OutputInterpAlpha);
					PVehicleOutput->Wheels[WheelIdx].WheelRadius = FMath::Lerp(Current.WheelRadius, Next.WheelRadius, OutputInterpAlpha);
					float DeltaAngle = FMath::FindDeltaAngleRadians(Current.AngularPosition, Next.AngularPosition);
					PVehicleOutput->Wheels[WheelIdx].AngularPosition = Current.AngularPosition + DeltaAngle * OutputInterpAlpha;
					PVehicleOutput->Wheels[WheelIdx].AngularVelocity = FMath::Lerp(Current.AngularVelocity, Next.AngularVelocity, OutputInterpAlpha);
					PVehicleOutput->Wheels[WheelIdx].LateralAdhesiveLimit = FMath::Lerp(Current.LateralAdhesiveLimit, Next.LateralAdhesiveLimit, OutputInterpAlpha);
					PVehicleOutput->Wheels[WheelIdx].LongitudinalAdhesiveLimit = FMath::Lerp(Current.LongitudinalAdhesiveLimit, Next.LongitudinalAdhesiveLimit, OutputInterpAlpha);

					PVehicleOutput->Wheels[WheelIdx].bIsSlipping = Current.bIsSlipping;
					PVehicleOutput->Wheels[WheelIdx].SlipMagnitude = FMath::Lerp(Current.SlipMagnitude, Next.SlipMagnitude, OutputInterpAlpha);
					PVehicleOutput->Wheels[WheelIdx].bIsSkidding = Current.bIsSkidding;
					PVehicleOutput->Wheels[WheelIdx].SkidMagnitude = FMath::Lerp(Current.SkidMagnitude, Next.SkidMagnitude, OutputInterpAlpha);
					PVehicleOutput->Wheels[WheelIdx].SkidNormal = FMath::Lerp(Current.SkidNormal, Next.SkidNormal, OutputInterpAlpha);
					PVehicleOutput->Wheels[WheelIdx].SlipAngle = FMath::Lerp(Current.SlipAngle, Next.SlipAngle, OutputInterpAlpha);

					PVehicleOutput->Wheels[WheelIdx].SuspensionOffset = FMath::Lerp(Current.SuspensionOffset, Next.SuspensionOffset, OutputInterpAlpha);
					PVehicleOutput->Wheels[WheelIdx].SpringForce = FMath::Lerp(Current.SpringForce, Next.SpringForce, OutputInterpAlpha);
					PVehicleOutput->Wheels[WheelIdx].NormalizedSuspensionLength = FMath::Lerp(Current.NormalizedSuspensionLength, Next.NormalizedSuspensionLength, OutputInterpAlpha);
					PVehicleOutput->Wheels[WheelIdx].DriveTorque = FMath::Lerp(Current.DriveTorque, Next.DriveTorque, OutputInterpAlpha);
					PVehicleOutput->Wheels[WheelIdx].BrakeTorque = FMath::Lerp(Current.BrakeTorque, Next.BrakeTorque, OutputInterpAlpha);

					PVehicleOutput->Wheels[WheelIdx].bABSActivated = Current.bABSActivated;
					PVehicleOutput->Wheels[WheelIdx].bBlockingHit = Current.bBlockingHit;
					PVehicleOutput->Wheels[WheelIdx].ImpactPoint = FMath::Lerp(Current.ImpactPoint, Next.ImpactPoint, OutputInterpAlpha);
					PVehicleOutput->Wheels[WheelIdx].HitLocation = FMath::Lerp(Current.HitLocation, Next.HitLocation, OutputInterpAlpha);
					PVehicleOutput->Wheels[WheelIdx].PhysMaterial = Current.PhysMaterial;
				}
			}
			else // WHEN ASYNC IS OFF IT STILL GENERATES THE ASYNC CALLBACK BUT THERE IS ONLY EVER THE CURRENT AND NO NEXT OUTPUT TO INTERPOLATE BETWEEN
			{
				PVehicleOutput->EngineRPM = CurAsyncOutput->VehicleSimOutput.EngineRPM;
				PVehicleOutput->EngineTorque = CurAsyncOutput->VehicleSimOutput.EngineTorque;
				PVehicleOutput->TransmissionRPM = CurAsyncOutput->VehicleSimOutput.TransmissionRPM;
				PVehicleOutput->TransmissionTorque = CurAsyncOutput->VehicleSimOutput.TransmissionTorque;

				for (int WheelIdx = 0; WheelIdx < CurrentOutput->VehicleSimOutput.Wheels.Num(); WheelIdx++)
				{
					const FWheelsOutput& Current = CurrentOutput->VehicleSimOutput.Wheels[WheelIdx];

					PVehicleOutput->Wheels[WheelIdx].InContact = Current.InContact;
					PVehicleOutput->Wheels[WheelIdx].SteeringAngle = Current.SteeringAngle;
					PVehicleOutput->Wheels[WheelIdx].WheelRadius = Current.WheelRadius;
					PVehicleOutput->Wheels[WheelIdx].AngularPosition = Current.AngularPosition;
					PVehicleOutput->Wheels[WheelIdx].AngularVelocity = Current.AngularVelocity;
					PVehicleOutput->Wheels[WheelIdx].LateralAdhesiveLimit = Current.LateralAdhesiveLimit;
					PVehicleOutput->Wheels[WheelIdx].LongitudinalAdhesiveLimit = Current.LongitudinalAdhesiveLimit;

					PVehicleOutput->Wheels[WheelIdx].bIsSlipping = Current.bIsSlipping;
					PVehicleOutput->Wheels[WheelIdx].SlipMagnitude = Current.SlipMagnitude;
					PVehicleOutput->Wheels[WheelIdx].bIsSkidding = Current.bIsSkidding;
					PVehicleOutput->Wheels[WheelIdx].SkidMagnitude = Current.SkidMagnitude;
					PVehicleOutput->Wheels[WheelIdx].SkidNormal = Current.SkidNormal;
					PVehicleOutput->Wheels[WheelIdx].SlipAngle = Current.SlipAngle;

					PVehicleOutput->Wheels[WheelIdx].SuspensionOffset = Current.SuspensionOffset;
					PVehicleOutput->Wheels[WheelIdx].SpringForce = Current.SpringForce;
					PVehicleOutput->Wheels[WheelIdx].NormalizedSuspensionLength = Current.NormalizedSuspensionLength;
					PVehicleOutput->Wheels[WheelIdx].DriveTorque = Current.DriveTorque;
					PVehicleOutput->Wheels[WheelIdx].BrakeTorque = Current.BrakeTorque;

					PVehicleOutput->Wheels[WheelIdx].bABSActivated = Current.bABSActivated;
					PVehicleOutput->Wheels[WheelIdx].bBlockingHit = Current.bBlockingHit;
					PVehicleOutput->Wheels[WheelIdx].ImpactPoint = Current.ImpactPoint;
					PVehicleOutput->Wheels[WheelIdx].HitLocation = Current.HitLocation;
					PVehicleOutput->Wheels[WheelIdx].PhysMaterial = Current.PhysMaterial;
				}

			}

		}
	}
}

void UChaosVehicleMovementComponent::GetBaseSnapshot(FBaseSnapshotData& SnapshotOut) const
{
	if (const FBodyInstance* TargetInstance = GetBodyInstance())
	{
		SnapshotOut.Transform = TargetInstance->GetUnrealWorldTransform();
		SnapshotOut.LinearVelocity = TargetInstance->GetUnrealWorldVelocity();
		SnapshotOut.AngularVelocity = TargetInstance->GetUnrealWorldAngularVelocityInRadians();
	}
}

void UChaosVehicleMovementComponent::SetBaseSnapshot(const FBaseSnapshotData& SnapshotIn)
{
	if (FBodyInstance* TargetInstance = GetBodyInstance())
	{
		TargetInstance->SetLinearVelocity(SnapshotIn.LinearVelocity, false);
		TargetInstance->SetAngularVelocityInRadians(SnapshotIn.AngularVelocity, false);
		TargetInstance->SetBodyTransform(SnapshotIn.Transform, ETeleportType::TeleportPhysics);
	}
}

void UChaosVehicleMovementComponent::WakeAllEnabledRigidBodies()
{
	if (USkeletalMeshComponent* Mesh = GetSkeletalMesh())
	{
		for (int32 i = 0; i < Mesh->Bodies.Num(); i++)
		{
			FBodyInstance* BI = Mesh->Bodies[i];
			check(BI);

			if (!BI->IsPhysicsDisabled() && BI->IsNonKinematic())
			{
				BI->WakeInstance();
			}
		}
	}
}

void UChaosVehicleMovementComponent::PutAllEnabledRigidBodiesToSleep()
{
	if (USkeletalMeshComponent* Mesh = GetSkeletalMesh())
	{
		for (int32 i = 0; i < Mesh->Bodies.Num(); i++)
		{
			FBodyInstance* BI = Mesh->Bodies[i];
			check(BI);

			if (!BI->IsPhysicsDisabled() && BI->IsNonKinematic())
			{
				BI->PutInstanceToSleep();
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE


#if VEHICLE_DEBUGGING_ENABLED
UE_ENABLE_OPTIMIZATION
#endif


