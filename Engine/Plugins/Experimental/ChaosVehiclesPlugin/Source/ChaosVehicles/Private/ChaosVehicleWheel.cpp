// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVehicleWheel.h"
#include "UObject/ConstructorHelpers.h"
#include "Engine/World.h"
#include "Engine/HitResult.h"
#include "Engine/Engine.h"
#include "Engine/StaticMesh.h"
#include "Vehicles/TireType.h"
#include "GameFramework/PawnMovementComponent.h"
#include "ChaosVehicleManager.h"
#include "ChaosWheeledVehicleMovementComponent.h"

#if VEHICLE_DEBUGGING_ENABLED
UE_DISABLE_OPTIMIZATION
#endif


UChaosVehicleWheel::UChaosVehicleWheel(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	static ConstructorHelpers::FObjectFinder<UStaticMesh> CollisionMeshObj(TEXT("/Engine/EngineMeshes/Cylinder"));
	CollisionMesh = CollisionMeshObj.Object;

	WheelRadius = 32.0f;
	WheelWidth = 20.0f;
	//bAutoAdjustCollisionSize = true;
	WheelMass = 20.0f;
	FrictionForceMultiplier = 2.0f;
	CorneringStiffness = 1000.0f;
	SideSlipModifier = 1.0f;
	SlipThreshold = 20.0f;
	SkidThreshold = 20.0f;
	MaxWheelspinRotation = 30;

	bAffectedByBrake = true;
	bAffectedByHandbrake = false;
	MaxSteerAngle = 50.0f;
	MaxBrakeTorque = 1500.f;
	MaxHandBrakeTorque = 3000.f;

	SpringRate = 250.0f;
	SpringPreload = 50.f;
	SuspensionAxis = FVector(0.f, 0.f, -1.f);
	SuspensionForceOffset = FVector::ZeroVector;
	SuspensionMaxRaise = 10.0f;
	SuspensionMaxDrop = 10.0f;
	SuspensionDampingRatio = 0.5f;
	SuspensionSmoothing = 0;
	WheelLoadRatio = 0.5f;
	RollbarScaling = 0.15f;
	SweepType = ESweepType::SimpleSweep;
	ExternalTorqueCombineMethod = ETorqueCombineMethod::None;
}


FChaosVehicleManager* UChaosVehicleWheel::GetVehicleManager() const
{
	UWorld* World = GEngine->GetWorldFromContextObject(VehicleComponent, EGetWorldErrorMode::LogAndReturnNull);
	return World ? FChaosVehicleManager::GetVehicleManagerFromScene(World->GetPhysicsScene()) : nullptr;
}


float UChaosVehicleWheel::GetSteerAngle() const
{
	check(VehicleComponent && VehicleComponent->PhysicsVehicleOutput());
	return VehicleComponent->PhysicsVehicleOutput()->Wheels[WheelIndex].SteeringAngle;
}

float UChaosVehicleWheel::GetRotationAngle() const
{
	check(VehicleComponent && VehicleComponent->PhysicsVehicleOutput());
	float RotationAngle = -1.0f * FMath::RadiansToDegrees(VehicleComponent->PhysicsVehicleOutput()->Wheels[WheelIndex].AngularPosition);
	ensure(!FMath::IsNaN(RotationAngle));
	return RotationAngle;
}

float UChaosVehicleWheel::GetRotationAngularVelocity() const
{
	check(VehicleComponent && VehicleComponent->PhysicsVehicleOutput());
	float RotationAngularVelocity = -1.0f * FMath::RadiansToDegrees(VehicleComponent->PhysicsVehicleOutput()->Wheels[WheelIndex].AngularVelocity);
	ensure(!FMath::IsNaN(RotationAngularVelocity));
	return RotationAngularVelocity;
}


float UChaosVehicleWheel::GetWheelRadius() const
{
	check(VehicleComponent && VehicleComponent->PhysicsVehicleOutput());
	return VehicleComponent->PhysicsVehicleOutput()->Wheels[WheelIndex].WheelRadius;
}

float UChaosVehicleWheel::GetWheelAngularVelocity() const
{
	check(VehicleComponent && VehicleComponent->PhysicsVehicleOutput());
	return VehicleComponent->PhysicsVehicleOutput()->Wheels[WheelIndex].AngularVelocity;
}

float UChaosVehicleWheel::GetSuspensionOffset() const
{
	check(VehicleComponent && VehicleComponent->PhysicsVehicleOutput());
	return VehicleComponent->GetSuspensionOffset(WheelIndex);
}

FVector UChaosVehicleWheel::GetSuspensionAxis() const
{
	check(VehicleComponent);
	return SuspensionAxis;
}

bool UChaosVehicleWheel::IsInAir() const
{
	check(VehicleComponent && VehicleComponent->PhysicsVehicleOutput());
	return !VehicleComponent->PhysicsVehicleOutput()->Wheels[WheelIndex].InContact;
}


void UChaosVehicleWheel::Init( UChaosWheeledVehicleMovementComponent* InVehicleSim, int32 InWheelIndex )
{
	check(InVehicleSim);
	check(InVehicleSim->Wheels.IsValidIndex(InWheelIndex));

	VehicleComponent = InVehicleSim;
	WheelIndex = InWheelIndex;

	Location = GetPhysicsLocation();
	OldLocation = Location;
}

void UChaosVehicleWheel::Shutdown()
{
//	WheelShape = NULL;
}

FChaosWheelSetup& UChaosVehicleWheel::GetWheelSetup()
{
	return VehicleComponent->WheelSetups[WheelIndex];
}

void UChaosVehicleWheel::Tick( float DeltaTime )
{
	OldLocation = Location;
	Location = GetPhysicsLocation();
	Velocity = ( Location - OldLocation ) / DeltaTime;
}

FVector UChaosVehicleWheel::GetPhysicsLocation()
{
	return Location;
}

#if WITH_EDITOR

void UChaosVehicleWheel::PostEditChangeProperty( FPropertyChangedEvent& PropertyChangedEvent )
{
	// Trigger a runtime rebuild of the Physics vehicle
	FChaosVehicleManager::VehicleSetupTag++;

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

#endif //WITH_EDITOR


UPhysicalMaterial* UChaosVehicleWheel::GetContactSurfaceMaterial()
{
	check(VehicleComponent && VehicleComponent->PhysicsVehicleOutput());
	return VehicleComponent->GetPhysMaterial(WheelIndex);
}


#if VEHICLE_DEBUGGING_ENABLED
UE_ENABLE_OPTIMIZATION
#endif



