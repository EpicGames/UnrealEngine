// Copyright Epic Games, Inc. All Rights Reserved.


#include "ChaosVehiclesPlugin.h"

//#include "AssetToolsModule.h"
#include "CoreMinimal.h"
#include "ChaosVehicles.h"
#include "HAL/ConsoleManager.h"
#include "Modules/ModuleManager.h"
#include "ChaosVehicleManager.h"


IMPLEMENT_MODULE( IChaosVehiclesPlugin, ChaosVehicles )


void IChaosVehiclesPlugin::PhysSceneInit(FPhysScene* PhysScene)
{
	new FChaosVehicleManager(PhysScene);
}

void IChaosVehiclesPlugin::PhysSceneTerm(FPhysScene* PhysScene)
{
	FChaosVehicleManager* VehicleManager = FChaosVehicleManager::GetVehicleManagerFromScene(PhysScene);
	if (VehicleManager != nullptr)
	{
		VehicleManager->DetachFromPhysScene(PhysScene);
		delete VehicleManager;
		VehicleManager = nullptr;
	}
}

void IChaosVehiclesPlugin::StartupModule()
{
	OnPhysSceneInitHandle = FPhysicsDelegates::OnPhysSceneInit.AddRaw(this, &IChaosVehiclesPlugin::PhysSceneInit);
	OnPhysSceneTermHandle = FPhysicsDelegates::OnPhysSceneTerm.AddRaw(this, &IChaosVehiclesPlugin::PhysSceneTerm);
}


void IChaosVehiclesPlugin::ShutdownModule()
{
	FPhysicsDelegates::OnPhysSceneInit.Remove(OnPhysSceneInitHandle);
	FPhysicsDelegates::OnPhysSceneTerm.Remove(OnPhysSceneTermHandle);
}

