// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosWheeledVehicleMovementComponent.h"
#include "Components/PrimitiveComponent.h"
#include "Components/SkinnedMeshComponent.h"
#include "Components/SkeletalMeshComponent.h"

#include "DrawDebugHelpers.h"
#include "DisplayDebugHelpers.h"
#include "Engine/Engine.h"
#include "Engine/HitResult.h"
#include "Engine/OverlapResult.h"
#include "GameFramework/Pawn.h"
#include "PhysicalMaterials/PhysicalMaterial.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "VehicleAnimationInstance.h"
#include "ChaosVehicleManager.h"
#include "ChaosVehicleWheel.h"
#include "SuspensionUtility.h"
#include "SteeringUtility.h"
#include "TransmissionUtility.h"
#include "Chaos/ChaosEngineInterface.h"
#include "Chaos/PBDSuspensionConstraintData.h"
#include "Chaos/DebugDrawQueue.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "PhysicsEngine/PhysicsObjectExternalInterface.h"

#include "PhysicsProxy/SuspensionConstraintProxy.h"
#include "PBDRigidsSolver.h"

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
#include "CanvasItem.h"
#include "Engine/Canvas.h"
#endif
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"

#if VEHICLE_DEBUGGING_ENABLED
UE_DISABLE_OPTIMIZATION
#endif

#if UE_WITH_REMOTE_OBJECT_HANDLE
#include "UObject/UObjectMigrationContext.h"
#endif

DECLARE_STATS_GROUP(TEXT("ChaosVehicle"), STATGROUP_ChaosVehicle, STATGROUP_Advanced);

DECLARE_CYCLE_STAT(TEXT("Vehicle:SuspensionRaycasts"), STAT_ChaosVehicle_SuspensionRaycasts, STATGROUP_ChaosVehicle);
DECLARE_CYCLE_STAT(TEXT("Vehicle:SuspensionOverlapTest"), STAT_ChaosVehicle_SuspensionOverlapTest, STATGROUP_ChaosVehicle);
DECLARE_CYCLE_STAT(TEXT("Vehicle:SuspensionTraces"), STAT_ChaosVehicle_SuspensionTraces, STATGROUP_ChaosVehicle);
DECLARE_CYCLE_STAT(TEXT("Vehicle:TickVehicle"), STAT_ChaosVehicle_TickVehicle, STATGROUP_ChaosVehicle);
DECLARE_CYCLE_STAT(TEXT("Vehicle:UpdateSimulation"), STAT_ChaosVehicle_UpdateSimulation, STATGROUP_ChaosVehicle);


FWheeledVehicleDebugParams GWheeledVehicleDebugParams;
extern FVehicleDebugParams GVehicleDebugParams;

EDebugPages UChaosWheeledVehicleMovementComponent::DebugPage = EDebugPages::BasicPage;

FAutoConsoleVariableRef CVarChaosVehiclesShowWheelCollisionNormal(TEXT("p.Vehicle.ShowWheelCollisionNormal"), GWheeledVehicleDebugParams.ShowWheelCollisionNormal, TEXT("Enable/Disable Wheel Collision Normal Visualisation."));
FAutoConsoleVariableRef CVarChaosVehiclesShowSuspensionRaycasts(TEXT("p.Vehicle.ShowSuspensionRaycasts"), GWheeledVehicleDebugParams.ShowSuspensionRaycasts, TEXT("Enable/Disable Suspension Raycast Visualisation."));
FAutoConsoleVariableRef CVarChaosVehiclesShowSuspensionLimits(TEXT("p.Vehicle.ShowSuspensionLimits"), GWheeledVehicleDebugParams.ShowSuspensionLimits, TEXT("Enable/Disable Suspension Limits Visualisation."));
FAutoConsoleVariableRef CVarChaosVehiclesShowWheelForces(TEXT("p.Vehicle.ShowWheelForces"), GWheeledVehicleDebugParams.ShowWheelForces, TEXT("Enable/Disable Wheel Forces Visualisation."));
FAutoConsoleVariableRef CVarChaosVehiclesShowSuspensionForces(TEXT("p.Vehicle.ShowSuspensionForces"), GWheeledVehicleDebugParams.ShowSuspensionForces, TEXT("Enable/Disable Suspension Forces Visualisation."));
FAutoConsoleVariableRef CVarChaosVehiclesShowBatchQueryExtents(TEXT("p.Vehicle.ShowBatchQueryExtents"), GWheeledVehicleDebugParams.ShowBatchQueryExtents, TEXT("Enable/Disable Suspension Forces Visualisation."));
FAutoConsoleVariableRef CVarChaosVehiclesShowRaycastComponent(TEXT("p.Vehicle.ShowRaycastComponent"), GWheeledVehicleDebugParams.ShowRaycastComponent, TEXT("Enable/Disable Raycast Component Hit Visualisation."));
FAutoConsoleVariableRef CVarChaosVehiclesShowRaycastMaterial(TEXT("p.Vehicle.ShowRaycastMaterial"), GWheeledVehicleDebugParams.ShowRaycastMaterial, TEXT("Enable/Disable Raycast Material Hit Visualisation."));
FAutoConsoleVariableRef CVarChaosVehiclesTraceTypeOverride(TEXT("p.Vehicle.TraceTypeOverride"), GWheeledVehicleDebugParams.TraceTypeOverride, TEXT("Override ray trace type, 1=Simple, 2=Complex."));
FAutoConsoleVariableRef CVarChaosVehiclesCacheSuspensionOffset(TEXT("p.Vehicle.CacheSuspensionOffset"), GWheeledVehicleDebugParams.CacheSuspensionOffset, TEXT("Cache Trace For Animation Optimization"));

FAutoConsoleVariableRef CVarChaosVehiclesDisableSuspensionForces(TEXT("p.Vehicle.DisableSuspensionForces"), GWheeledVehicleDebugParams.DisableSuspensionForces, TEXT("Enable/Disable Suspension Forces."));
FAutoConsoleVariableRef CVarChaosVehiclesDisableFrictionForces(TEXT("p.Vehicle.DisableFrictionForces"), GWheeledVehicleDebugParams.DisableFrictionForces, TEXT("Enable/Disable Wheel Friction Forces."));
FAutoConsoleVariableRef CVarChaosVehiclesDisableRollbarForces(TEXT("p.Vehicle.DisableRollbarForces"), GWheeledVehicleDebugParams.DisableRollbarForces, TEXT("Enable/Disable Rollbar Forces."));
FAutoConsoleVariableRef CVarChaosVehiclesDisableConstraintSuspension(TEXT("p.Vehicle.DisableConstraintSuspension"), GWheeledVehicleDebugParams.DisableConstraintSuspension, TEXT("Enable/Disable Constraint based suspension, swaps to basic force based suspension without hardstops instead."));

FAutoConsoleVariableRef CVarChaosVehiclesThrottleOverride(TEXT("p.Vehicle.ThrottleOverride"), GWheeledVehicleDebugParams.ThrottleOverride, TEXT("Hard code throttle input on."));
FAutoConsoleVariableRef CVarChaosVehiclesSteeringOverride(TEXT("p.Vehicle.SteeringOverride"), GWheeledVehicleDebugParams.SteeringOverride, TEXT("Hard code steering input on."));

FAutoConsoleVariableRef CVarChaosVehiclesResetMeasurements(TEXT("p.Vehicle.ResetMeasurements"), GWheeledVehicleDebugParams.ResetPerformanceMeasurements, TEXT("Reset Vehicle Performance Measurements."));

FAutoConsoleVariableRef CVarChaosVehiclesOverlapTestExpansionXY(TEXT("p.Vehicle.OverlapTestExpansionXY"), GWheeledVehicleDebugParams.OverlapTestExpansionXY, TEXT("Raycast Overlap Test Expansion of Bounding Box in X/Y axes."));
FAutoConsoleVariableRef CVarChaosVehiclesOverlapTestExpansionXZ(TEXT("p.Vehicle.OverlapTestExpansionZ"), GWheeledVehicleDebugParams.OverlapTestExpansionZ, TEXT("Raycast Overlap Test Expansion of Bounding Box in Z axis"));

//FAutoConsoleVariableRef CVarChaosVehiclesDisableSuspensionConstraints(TEXT("p.Vehicle.DisableSuspensionConstraint"), GWheeledVehicleDebugParams.DisableSuspensionConstraint, TEXT("Enable/Disable Suspension Constraints."));

FAutoConsoleCommand CVarCommandVehiclesNextDebugPage(
	TEXT("p.Vehicle.NextDebugPage"),
	TEXT("Display the next page of vehicle debug data."),
	FConsoleCommandDelegate::CreateStatic(UChaosWheeledVehicleMovementComponent::NextDebugPage));

FAutoConsoleCommand CVarCommandVehiclesPrevDebugPage(
	TEXT("p.Vehicle.PrevDebugPage"),
	TEXT("Display the previous page of vehicle debug data."),
	FConsoleCommandDelegate::CreateStatic(UChaosWheeledVehicleMovementComponent::PrevDebugPage));


FString FWheelStatus::ToString() const
{
	return FString::Printf(TEXT("bInContact:%s ContactPoint:%s PhysMaterial:%s NormSuspensionLength:%f SpringForce:%f SlipAngle:%f bIsSlipping:%s SlipMagnitude:%f bIsSkidding:%s SkidMagnitude:%f SkidNormal:%s DriveTorque:%f BrakeTorque:%f ABSActive:%s"),
		bInContact == true ? TEXT("True") : TEXT("False"),
		*ContactPoint.ToString(),
		PhysMaterial.IsValid() ? *PhysMaterial->GetName() : TEXT("None"),
		NormalizedSuspensionLength,
		SpringForce,
		SlipAngle,
		bIsSlipping == true ? TEXT("True") : TEXT("False"),
		SlipMagnitude,
		bIsSkidding == true ? TEXT("True") : TEXT("False"),
		SkidMagnitude,
		*SkidNormal.ToString(),
		DriveTorque,
		BrakeTorque,
		bABSActivated == true ? TEXT("True") : TEXT("False"));
}

void FWheelState::CaptureState(int WheelIdx, const FVector& WheelOffset, const FBodyInstance* TargetInstance)
{
	check(TargetInstance);
	const FTransform WorldTransform = TargetInstance->GetUnrealWorldTransform();
	WheelLocalLocation[WheelIdx] = WheelOffset;
	WheelWorldLocation[WheelIdx] = WorldTransform.TransformPosition(WheelOffset);
	WorldWheelVelocity[WheelIdx] = TargetInstance->GetUnrealWorldVelocityAtPoint(WheelWorldLocation[WheelIdx]);
	LocalWheelVelocity[WheelIdx] = WorldTransform.InverseTransformVector(WorldWheelVelocity[WheelIdx]);
}

void FWheelState::CaptureState(int WheelIdx, const FVector& WheelOffset, const Chaos::FRigidBodyHandle_Internal* Handle)
{
	check(Handle);
	const FTransform WorldTransform(Handle->R(), Handle->X());
	WheelLocalLocation[WheelIdx] = WheelOffset;
	WheelWorldLocation[WheelIdx] = WorldTransform.TransformPosition(WheelOffset);
	WorldWheelVelocity[WheelIdx] = GetVelocityAtPoint(Handle, WheelWorldLocation[WheelIdx]);
	LocalWheelVelocity[WheelIdx] = WorldTransform.InverseTransformVector(WorldWheelVelocity[WheelIdx]);
}

void FWheelState::CaptureState(int WheelIdx, const FVector& WheelOffset, const Chaos::FRigidBodyHandle_Internal* VehicleHandle, const FVector& ContactPoint, const Chaos::FRigidBodyHandle_Internal* SurfaceHandle)
{
	check(VehicleHandle);

	FVector SurfaceVelocity = FVector::ZeroVector;
	if (SurfaceHandle)
	{
		SurfaceVelocity = GetVelocityAtPoint(SurfaceHandle, ContactPoint);
	}

	const FTransform WorldTransform(VehicleHandle->R(), VehicleHandle->X());
	WheelLocalLocation[WheelIdx] = WheelOffset;
	WheelWorldLocation[WheelIdx] = WorldTransform.TransformPosition(WheelOffset);
	WorldWheelVelocity[WheelIdx] = GetVelocityAtPoint(VehicleHandle, WheelWorldLocation[WheelIdx]) - SurfaceVelocity;
	LocalWheelVelocity[WheelIdx] = WorldTransform.InverseTransformVector(WorldWheelVelocity[WheelIdx]);
}

FVector FWheelState::GetVelocityAtPoint(const Chaos::FRigidBodyHandle_Internal* Rigid, const FVector& InPoint)
{
	if (Rigid)
	{
		const Chaos::FVec3 COM = Rigid ? Chaos::FParticleUtilitiesGT::GetCoMWorldPosition(Rigid) : (Chaos::FVec3)Chaos::FParticleUtilitiesGT::GetActorWorldTransform(Rigid).GetTranslation();
		const Chaos::FVec3 Diff = InPoint - COM;
		return Rigid->V() - Chaos::FVec3::CrossProduct(Diff, Rigid->W());
	}
	else
	{
		return FVector::ZeroVector;
	}
}

/**
 * UChaosWheeledVehicleSimulation
 */
bool UChaosWheeledVehicleSimulation::CanSimulate() const
{
	if (UChaosVehicleSimulation::CanSimulate() == false)
	{
		return false;
	}

	return (PVehicle && PVehicle.IsValid()
		&& PVehicle->Engine.Num() == PVehicle->Transmission.Num()
		&& PVehicle->Wheels.Num() == PVehicle->Suspension.Num());
}

void UChaosWheeledVehicleSimulation::TickVehicle(UWorld* WorldIn, float DeltaTime, const FChaosVehicleAsyncInput& InputData, FChaosVehicleAsyncOutput& OutputData, Chaos::FRigidBodyHandle_Internal* Handle)
{
	SCOPE_CYCLE_COUNTER(STAT_ChaosVehicle_TickVehicle);

	UChaosVehicleSimulation::TickVehicle(WorldIn, DeltaTime, InputData, OutputData, Handle);
}


void UChaosWheeledVehicleSimulation::UpdateState(float DeltaTime, const FChaosVehicleAsyncInput& InputData, Chaos::FRigidBodyHandle_Internal* Handle)
{
	UChaosVehicleSimulation::UpdateState(DeltaTime, InputData, Handle);

	if (CanSimulate() && Handle)
	{
		// sanity check that everything is setup ok
		ensure(PVehicle->Wheels.Num() == PVehicle->Suspension.Num());
		ensure(WheelState.LocalWheelVelocity.Num() == PVehicle->Wheels.Num());
		ensure(WheelState.WheelWorldLocation.Num() == PVehicle->Wheels.Num());
		ensure(WheelState.WorldWheelVelocity.Num() == PVehicle->Wheels.Num());

		///////////////////////////////////////////////////////////////////////
		// Cache useful state so we are not re-calculating the same data
		for (int WheelIdx = 0; WheelIdx < PVehicle->Suspension.Num(); WheelIdx++)
		{
			bool bCaptured = false;

			// #TODO: This is not threadsafe - need to rethink how to get the rigidbody that is hit by the raycast
			//const FHitResult& HitResult = Wheels[WheelIdx]->HitResult;
			//if (HitResult.Component.IsValid() && HitResult.Component->GetBodyInstance())
			//{
			//	if (const FPhysicsActorHandle& SurfaceHandle = HitResult.Component->GetBodyInstance()->GetPhysicsActor())
			//	{
			//		// we are being called from the physics thread
			//		if (Chaos::FRigidBodyHandle_Internal* SurfaceBody = SurfaceHandle->GetPhysicsThreadAPI())
			//		{
			//			if (SurfaceBody->CanTreatAsKinematic())
			//			{
			//				FVector Point = HitResult.ImpactPoint;
			//				WheelState.CaptureState(WheelIdx, PVehicle->Suspension[WheelIdx].GetLocalRestingPosition(), Handle, Point, SurfaceBody);
			//				bCaptured = true;
			//			}
			//		}
			//	}
			//}

			if (!bCaptured)
			{
				WheelState.CaptureState(WheelIdx, PVehicle->Suspension[WheelIdx].GetLocalRestingPosition(), Handle);
			}
		}
		///////////////////////////////////////////////////////////////////////
		// Suspension Raycast

		for (int WheelIdx = 0; WheelIdx < PVehicle->Suspension.Num(); WheelIdx++)
		{
			auto& PSuspension = PVehicle->Suspension[WheelIdx];
			auto& PWheel = PVehicle->Wheels[WheelIdx];
			PSuspension.UpdateWorldRaycastLocation(VehicleState.VehicleWorldTransform, PWheel.GetEffectiveRadius(), WheelState.Trace[WheelIdx]);
		}

		if (!GWheeledVehicleDebugParams.DisableSuspensionForces && PVehicle->bSuspensionEnabled)
		{
			PerformSuspensionTraces(WheelState.Trace, InputData.PhysicsInputs.TraceParams, InputData.PhysicsInputs.TraceCollisionResponse, InputData.PhysicsInputs.WheelTraceParams);
		}

		//////////////////////////////////////////////////////////////////////////
		// Wheel and Vehicle in air state

		VehicleState.bVehicleInAir = true;
		VehicleState.NumWheelsOnGround = 0;
		for (int WheelIdx = 0; WheelIdx < PVehicle->Wheels.Num(); WheelIdx++)
		{
			// tell systems who care that wheel is touching the ground
			PVehicle->Wheels[WheelIdx].SetOnGround(WheelState.TraceResult[WheelIdx].bBlockingHit);

			// only requires one wheel to be on the ground for the vehicle to be NOT in the air
			if (PVehicle->Wheels[WheelIdx].InContact())
			{
				VehicleState.bVehicleInAir = false;
				VehicleState.NumWheelsOnGround++;
			}
		}
		VehicleState.bAllWheelsOnGround = (VehicleState.NumWheelsOnGround == PVehicle->Wheels.Num());
	}
}

void UChaosWheeledVehicleSimulation::UpdateSimulation(float DeltaTime, const FChaosVehicleAsyncInput& InputData, Chaos::FRigidBodyHandle_Internal* Handle)
{
	SCOPE_CYCLE_COUNTER(STAT_ChaosVehicle_UpdateSimulation);
	
	// Inherit common vehicle simulation stages ApplyAerodynamics, ApplyTorqueControl, etc
	UChaosVehicleSimulation::UpdateSimulation(DeltaTime, InputData, Handle);

	if (CanSimulate() && Handle)
	{
		///////////////////////////////////////////////////////////////////////
		// Engine/Transmission
		if (!GWheeledVehicleDebugParams.DisableSuspensionForces && PVehicle->bMechanicalSimEnabled)
		{
			ProcessMechanicalSimulation(DeltaTime);
		}

		///////////////////////////////////////////////////////////////////////
		// Suspension

		if (!GWheeledVehicleDebugParams.DisableSuspensionForces && PVehicle->bSuspensionEnabled)
		{
			ApplySuspensionForces(DeltaTime, InputData.PhysicsInputs.WheelTraceParams);
		}

		///////////////////////////////////////////////////////////////////////
		// Steering

		ProcessSteering(InputData.PhysicsInputs.NetworkInputs.VehicleInputs);

		///////////////////////////////////////////////////////////////////////
		// Wheel Friction

		if (!GWheeledVehicleDebugParams.DisableFrictionForces && PVehicle->bWheelFrictionEnabled)
		{
			ApplyWheelFrictionForces(DeltaTime);
		}

#if 0
		if (PerformanceMeasure.IsEnabled())
		{
			PerformanceMeasure.Update(DeltaTime, VehicleState.VehicleWorldTransform.GetLocation(), VehicleState.ForwardSpeed);
		}
#endif
	}
}

bool UChaosWheeledVehicleSimulation::ContainsTraces(const FBox& Box, const TArray<Chaos::FSuspensionTrace>& SuspensionTrace)
{
	const Chaos::FAABB3 Aabb(Box.Min, Box.Max);

	for (int WheelIdx = 0; WheelIdx < SuspensionTrace.Num(); WheelIdx++)
	{
		if (!Aabb.Contains(SuspensionTrace[WheelIdx].Start) || !Aabb.Contains(SuspensionTrace[WheelIdx].End))
		{
			return false;
		}
	}

	return true;
}

void UChaosWheeledVehicleSimulation::PerformSuspensionTraces(const TArray<Chaos::FSuspensionTrace>& SuspensionTrace, FCollisionQueryParams& TraceParams, FCollisionResponseContainer& CollisionResponse, TArray<FWheelTraceParams>& WheelTraceParams)
{
	SCOPE_CYCLE_COUNTER(STAT_ChaosVehicle_SuspensionRaycasts);

	ECollisionChannel SpringCollisionChannel = ECollisionChannel::ECC_WorldDynamic;
	FCollisionResponseParams ResponseParams;
	ResponseParams.CollisionResponse = CollisionResponse;

	// batching is about 0.5ms (25%) faster when there's 100 vehicles on a flat terrain
	if (GVehicleDebugParams.BatchQueries)
	{
		if (!GVehicleDebugParams.CacheTraceOverlap || !ContainsTraces(QueryBox, SuspensionTrace))
		{
			SCOPE_CYCLE_COUNTER(STAT_ChaosVehicle_SuspensionOverlapTest);

			bOverlapHit = false;
			OverlapResults.Empty();
			QueryBox.Init();

			//FBox QueryBox;
			for (int WheelIdx = 0; WheelIdx < SuspensionTrace.Num(); WheelIdx++)
			{
				const FVector& TraceStart = SuspensionTrace[WheelIdx].Start;
				const FVector& TraceEnd = SuspensionTrace[WheelIdx].End;

				if (WheelIdx == 0)
				{
					QueryBox = FBox(TraceStart, TraceEnd);
				}
				else
				{
					QueryBox.Min = QueryBox.Min.ComponentMin(TraceStart);
					QueryBox.Min = QueryBox.Min.ComponentMin(TraceEnd);
					QueryBox.Max = QueryBox.Max.ComponentMax(TraceStart);
					QueryBox.Max = QueryBox.Max.ComponentMax(TraceEnd);
				}
			}
			QueryBox = QueryBox.ExpandBy(FVector(GWheeledVehicleDebugParams.OverlapTestExpansionXY, GWheeledVehicleDebugParams.OverlapTestExpansionXY, GWheeledVehicleDebugParams.OverlapTestExpansionZ));
			FCollisionShape CollisionBox;
			CollisionBox.SetBox((FVector3f)QueryBox.GetExtent());

			bOverlapHit = World->OverlapMultiByChannel(OverlapResults, QueryBox.GetCenter(), FQuat::Identity, SpringCollisionChannel, CollisionBox, TraceParams, ResponseParams);
		}
		
	#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (GWheeledVehicleDebugParams.ShowBatchQueryExtents)
		{
			Chaos::FDebugDrawQueue::GetInstance().DrawDebugBox(QueryBox.GetCenter(), QueryBox.GetExtent(), FQuat::Identity, FColor::Yellow, false, -1.0f, 0, 2.0f);

			// draw all corresponding results bounding boxes
			for (FOverlapResult OverlapResult : OverlapResults)
			{
				if (OverlapResult.bBlockingHit)
				{
					const FBoxSphereBounds Bounds = OverlapResult.Component->CalcBounds(OverlapResult.Component->GetComponentTransform());
					Chaos::FDebugDrawQueue::GetInstance().DrawDebugBox(Bounds.GetBox().GetCenter(), Bounds.GetBox().GetExtent(), FQuat::Identity, FColor::Purple, false, -1.0f, 0, 2.0f);
				}
			}
		}
	#endif

		SCOPE_CYCLE_COUNTER(STAT_ChaosVehicle_SuspensionTraces);
		for (int32 WheelIdx = 0; WheelIdx < SuspensionTrace.Num(); ++WheelIdx)
		{
			FHitResult& HitResult = WheelState.TraceResult[WheelIdx];
			HitResult = FHitResult();

			if (bOverlapHit)
			{
				const FVector& TraceStart = SuspensionTrace[WheelIdx].Start;
				const FVector& TraceEnd = SuspensionTrace[WheelIdx].End;
				TraceParams.bTraceComplex = (WheelTraceParams[WheelIdx].SweepType == ESweepType::ComplexSweep);

				if (GWheeledVehicleDebugParams.TraceTypeOverride > 0)
				{
					TraceParams.bTraceComplex = GWheeledVehicleDebugParams.TraceTypeOverride == 2;
				}

				FVector TraceVector(TraceStart - TraceEnd); // reversed
				FVector TraceNormal = TraceVector.GetSafeNormal();

				// Test each overlapped object for a hit result
				for (FOverlapResult OverlapResult : OverlapResults)
				{
					if (!OverlapResult.bBlockingHit)
						continue;

					FHitResult ComponentHit;

					switch (WheelTraceParams[WheelIdx].SweepShape)
					{
					case ESweepShape::Spherecast:
					{
						float WheelRadius = PVehicle->Wheels[WheelIdx].GetEffectiveRadius();
						FVector VehicleUpAxis = TraceNormal;

						FVector Start = TraceStart + VehicleUpAxis * WheelRadius;
						FVector End = TraceEnd;

						if (OverlapResult.Component->SweepComponent(ComponentHit, Start, End, FQuat::Identity, FCollisionShape::MakeSphere(WheelRadius), TraceParams.bTraceComplex))
						{
							if (ComponentHit.Time < HitResult.Time)
							{
								HitResult = ComponentHit;
								HitResult.bBlockingHit = OverlapResult.bBlockingHit;
							}
						}
					}
					break;

					case ESweepShape::Raycast:
					default:
					{
						if (OverlapResult.Component->LineTraceComponent(ComponentHit, TraceStart, TraceEnd, TraceParams))
						{
							if (ComponentHit.Time < HitResult.Time)
							{
								HitResult = ComponentHit;
								HitResult.bBlockingHit = OverlapResult.bBlockingHit;
							}
						}
					}
					break;
					}
				}
			}
		}
	}
	else
	{
		SCOPE_CYCLE_COUNTER(STAT_ChaosVehicle_SuspensionTraces);
		for (int WheelIdx = 0; WheelIdx < SuspensionTrace.Num(); WheelIdx++)
		{
			FHitResult& HitResult = WheelState.TraceResult[WheelIdx];

			FVector TraceStart = SuspensionTrace[WheelIdx].Start;
			FVector TraceEnd = SuspensionTrace[WheelIdx].End;
			TraceParams.bTraceComplex = (WheelTraceParams[WheelIdx].SweepType == ESweepType::ComplexSweep);

			if (GWheeledVehicleDebugParams.TraceTypeOverride > 0)
			{
				TraceParams.bTraceComplex = GWheeledVehicleDebugParams.TraceTypeOverride == 2;
			}

			FVector TraceVector(TraceStart - TraceEnd); // reversed
			FVector TraceNormal = TraceVector.GetSafeNormal();

			switch (WheelTraceParams[WheelIdx].SweepShape)
			{
			case ESweepShape::Spherecast:
			{
				float WheelRadius = PVehicle->Wheels[WheelIdx].GetEffectiveRadius();
				FVector VehicleUpAxis = TraceNormal;

				World->SweepSingleByChannel(HitResult
					, TraceStart + VehicleUpAxis * WheelRadius
					, TraceEnd
					, FQuat::Identity, SpringCollisionChannel
					, FCollisionShape::MakeSphere(WheelRadius), TraceParams
					, ResponseParams);
			}
			break;



			case ESweepShape::Raycast:
			default:
			{
				World->LineTraceSingleByChannel(HitResult, TraceStart, TraceEnd, SpringCollisionChannel, TraceParams, ResponseParams);
			}
			break;
			}
		}
	}

}

void UChaosWheeledVehicleSimulation::ApplyWheelFrictionForces(float DeltaTime)
{
	using namespace Chaos;

	for (int WheelIdx = 0; WheelIdx < PVehicle->Wheels.Num(); WheelIdx++)
	{
		auto& PWheel = PVehicle->Wheels[WheelIdx]; // Physics Wheel
		FHitResult& HitResult = WheelState.TraceResult[WheelIdx];

		if (PWheel.InContact())
		{
			if (HitResult.PhysMaterial.IsValid())
			{
				PWheel.SetSurfaceFriction(HitResult.PhysMaterial->Friction);
			}

			// take into account steering angle
			float SteerAngleDegrees = PWheel.SteeringAngle;
			FRotator SteeringRotator(0.f, SteerAngleDegrees, 0.f);
			FVector SteerLocalWheelVelocity = SteeringRotator.UnrotateVector(WheelState.LocalWheelVelocity[WheelIdx]);

			PWheel.SetVehicleGroundSpeed(SteerLocalWheelVelocity);
			PWheel.Simulate(DeltaTime);

			float RotationAngle = FMath::RadiansToDegrees(PWheel.GetAngularPosition());
			FVector FrictionForceLocal = PWheel.GetForceFromFriction();
			FrictionForceLocal = SteeringRotator.RotateVector(FrictionForceLocal);

			FVector GroundZVector = HitResult.Normal;
			FVector GroundXVector = FVector::CrossProduct(VehicleState.VehicleRightAxis, GroundZVector);
			FVector GroundYVector = FVector::CrossProduct(GroundZVector, GroundXVector);
			
			FMatrix Mat = FMatrix(GroundXVector, GroundYVector, GroundZVector, VehicleState.VehicleWorldTransform.GetLocation());
			FVector FrictionForceVector = Mat.TransformVector(FrictionForceLocal);

			check(PWheel.InContact());
			if (PVehicle->bLegacyWheelFrictionPosition)
			{
				AddForceAtPosition(FrictionForceVector, WheelState.WheelWorldLocation[WheelIdx]);
			}
			else
			{
				AddForceAtPosition(FrictionForceVector, HitResult.ImpactPoint);
			}
		
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			if (GWheeledVehicleDebugParams.ShowWheelForces)
			{
				// show longitudinal drive force
				if (PWheel.AvailableGrip > 0.0f)
				{
					float Radius = 50.0f;
					float Scaling = 50.0f / PWheel.AvailableGrip;

					FVector Center = WheelState.WheelWorldLocation[WheelIdx];			
					FVector Offset(0.0f, WheelState.WheelLocalLocation[WheelIdx].Y, 10.f);
					Offset = Mat.TransformVector(Offset);

					FDebugDrawQueue::GetInstance().DrawDebugLine(Center, Center + GroundZVector * 100.f, FColor::Orange, false, -1.0f, 0, 2);

					Center += Offset;
					FVector InputForceVectorWorld = Mat.TransformVector(PWheel.InputForces);
					FDebugDrawQueue::GetInstance().DrawDebugCircle(Center, Radius, 60, FColor::White, false, -1.0f, 0, 3, FVector(1,0,0), FVector(0,1,0), false);
					FDebugDrawQueue::GetInstance().DrawDebugLine(Center, Center + InputForceVectorWorld * Scaling, (PWheel.bClipping?FColor::Red:FColor::Green), false, -1.0f, 0, PWheel.bClipping?2:4);
					FDebugDrawQueue::GetInstance().DrawDebugLine(Center, Center + FrictionForceVector * Scaling, FColor::Yellow, false, -1.0f, 1, PWheel.bClipping?4:2);

				}

			}
#endif

		}
		else
		{
			PWheel.SetVehicleGroundSpeed(FVector::ZeroVector);
			PWheel.SetWheelLoadForce(0.f);
			PWheel.Simulate(DeltaTime);
		}

	}
}

void UChaosWheeledVehicleSimulation::ApplySuspensionForces(float DeltaTime, TArray<FWheelTraceParams>& WheelTraceParams)
{
	using namespace Chaos;

	TArray<float> SusForces;
	SusForces.Init(0.f, PVehicle->Suspension.Num());

	for (int WheelIdx = 0; WheelIdx < SusForces.Num(); WheelIdx++)
	{
		FHitResult& HitResult = WheelState.TraceResult[WheelIdx];

		float NewDesiredLength = 1.0f; // suspension max length
		float ForceMagnitude2 = 0.f;
		auto& PWheel = PVehicle->Wheels[WheelIdx];
		auto& PSuspension = PVehicle->Suspension[WheelIdx];
		float SuspensionMovePosition = -PSuspension.Setup().MaxLength;

		if (!GWheeledVehicleDebugParams.DisableConstraintSuspension)
		{
			if (WheelIdx < ConstraintHandles.Num())
			{
				FPhysicsConstraintHandle& ConstraintHandle = ConstraintHandles[WheelIdx];
				if (ConstraintHandle.IsValid())
				{
					if (Chaos::FSuspensionConstraint* Constraint = static_cast<Chaos::FSuspensionConstraint*>(ConstraintHandle.Constraint))
					{
						if (FSuspensionConstraintPhysicsProxy* Proxy = Constraint->GetProxy<FSuspensionConstraintPhysicsProxy>())
						{
							FVec3 TargetPos;
							if (WheelTraceParams[WheelIdx].SweepShape == ESweepShape::Spherecast)
							{
								TargetPos = HitResult.Location;
							}
							else
							{
								TargetPos = HitResult.ImpactPoint + (PWheel.GetEffectiveRadius() * VehicleState.VehicleUpAxis);
							}

							Chaos::FPhysicsSolver* Solver = Proxy->GetSolver<Chaos::FPhysicsSolver>();

							IPhysicsProxyBase* HitProxy = nullptr;

							if (Chaos::FPhysicsObjectHandle PhysicsObject = HitResult.PhysicsObject)
							{
								HitProxy = Chaos::FPhysicsObjectInterface::GetProxy({ &PhysicsObject, 1 });
							}

							Solver->SetSuspensionTarget(Constraint, TargetPos, HitResult.Normal, PWheel.InContact(), HitProxy);
						}
					}
				}
			}
		}

		if (PWheel.InContact())
		{
			NewDesiredLength = HitResult.Distance;

			SuspensionMovePosition = -FVector::DotProduct(WheelState.WheelWorldLocation[WheelIdx] - HitResult.ImpactPoint, VehicleState.VehicleUpAxis) + PWheel.GetEffectiveRadius();

			PSuspension.SetSuspensionLength(NewDesiredLength, PWheel.GetEffectiveRadius());
			PSuspension.SetLocalVelocity(WheelState.LocalWheelVelocity[WheelIdx]);
			PSuspension.Simulate(DeltaTime);

			float ForceMagnitude = PSuspension.GetSuspensionForce();

			FVector GroundZVector = HitResult.Normal;
			FVector SuspensionForceVector = VehicleState.VehicleUpAxis * ForceMagnitude;

			FVector SusApplicationPoint = WheelState.WheelWorldLocation[WheelIdx] + PVehicle->Suspension[WheelIdx].Setup().SuspensionForceOffset;

			check(PWheel.InContact());
			if (GWheeledVehicleDebugParams.DisableConstraintSuspension)
			{
				AddForceAtPosition(SuspensionForceVector, SusApplicationPoint);
			}

			ForceMagnitude = PSuspension.Setup().WheelLoadRatio * ForceMagnitude + (1.f - PSuspension.Setup().WheelLoadRatio) * PSuspension.Setup().RestingForce;
			PWheel.SetWheelLoadForce(ForceMagnitude);
			PWheel.SetMassPerWheel(RigidHandle->M() / PVehicle->Wheels.Num());
			SusForces[WheelIdx] = ForceMagnitude;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			if (GWheeledVehicleDebugParams.ShowSuspensionForces)
			{
				Chaos::FDebugDrawQueue::GetInstance().DrawDebugLine(
					  SusApplicationPoint
					, SusApplicationPoint + SuspensionForceVector * GVehicleDebugParams.ForceDebugScaling
					, FColor::Blue, false, -1.0f, 0, 5);

				Chaos::FDebugDrawQueue::GetInstance().DrawDebugLine(
					  SusApplicationPoint
					, SusApplicationPoint + GroundZVector * 140.f
					, FColor::Yellow, false, -1.0f, 0, 5);
			}
#endif

		}
		else
		{
			PSuspension.SetSuspensionLength(PSuspension.GetTraceLength(PWheel.GetEffectiveRadius()), PWheel.Setup().WheelRadius);
			PWheel.SetWheelLoadForce(0.f);

		}

	}

	if (!GWheeledVehicleDebugParams.DisableRollbarForces)
	{
		for (auto& Axle : PVehicle->GetAxles())
		{
			//#todo: only works with 2 wheels on an axle at present
			if (Axle.Setup.WheelIndex.Num() == 2)
			{
				uint16 WheelIdxA = Axle.Setup.WheelIndex[0];
				uint16 WheelIdxB = Axle.Setup.WheelIndex[1];

				float FV = Axle.Setup.RollbarScaling;
				float ForceDiffOnAxleF = SusForces[WheelIdxA] - SusForces[WheelIdxB];
				FVector ForceVector0 = VehicleState.VehicleUpAxis * ForceDiffOnAxleF * FV;
				FVector ForceVector1 = VehicleState.VehicleUpAxis * ForceDiffOnAxleF * -FV;

				FVector SusApplicationPoint0 = WheelState.WheelWorldLocation[WheelIdxA] + PVehicle->Suspension[WheelIdxA].Setup().SuspensionForceOffset;
				AddForceAtPosition(ForceVector0, SusApplicationPoint0);

				FVector SusApplicationPoint1 = WheelState.WheelWorldLocation[WheelIdxB] + PVehicle->Suspension[WheelIdxB].Setup().SuspensionForceOffset;
				AddForceAtPosition(ForceVector1, SusApplicationPoint1);
			}
		}
	}
}

void UChaosWheeledVehicleSimulation::ProcessSteering(const FControlInputs& ControlInputs)
{
	using namespace Chaos;

	auto& PSteering = PVehicle->GetSteering();

	for (int WheelIdx = 0; WheelIdx < PVehicle->Wheels.Num(); WheelIdx++)
	{
		auto& PWheel = PVehicle->Wheels[WheelIdx]; // Physics Wheel
		FHitResult& HitResult = WheelState.TraceResult[WheelIdx];

		if (PWheel.SteeringEnabled)
		{
			float SpeedScale = 1.0f;

			// allow full counter steering when steering into a power slide
			//if (ControlInputs.SteeringInput * VehicleState.VehicleLocalVelocity.Y > 0.0f)
			{
				SpeedScale = PVehicle->GetSteering().GetSteeringFromVelocity(CmSToMPH(VehicleState.ForwardSpeed));
			}

			float SteeringAngle = ControlInputs.SteeringInput * SpeedScale;

			if (FMath::Abs(GWheeledVehicleDebugParams.SteeringOverride) > 0.01f)
			{
				SteeringAngle = PWheel.MaxSteeringAngle * GWheeledVehicleDebugParams.SteeringOverride;
			}
			else
			{
				float WheelSide = PVehicle->GetSuspension(WheelIdx).GetLocalRestingPosition().Y;
				SteeringAngle = PSteering.GetSteeringAngle(SteeringAngle, PWheel.MaxSteeringAngle, WheelSide);
			}

			PWheel.SetSteeringAngle(SteeringAngle);
		}
		else
		{
			PWheel.SetSteeringAngle(0.0f);
		}
	}
}

void UChaosWheeledVehicleSimulation::ApplyInput(const FControlInputs& ControlInputs, float DeltaTime)
{
	using namespace Chaos;

	UChaosVehicleSimulation::ApplyInput(ControlInputs, DeltaTime);

	FControlInputs ModifiedInputs = ControlInputs;

	float EngineBraking = 0.f;
	if (PVehicle->HasTransmission() && PVehicle->HasEngine())
	{
		auto& PEngine = PVehicle->GetEngine();
		auto& PTransmission = PVehicle->GetTransmission();

		if (ModifiedInputs.TransmissionType != PTransmission.Setup().TransmissionType)
		{
			PTransmission.AccessSetup().TransmissionType = (Chaos::ETransmissionType)ModifiedInputs.TransmissionType;
		}

		if (ModifiedInputs.GearUpInput)
		{
			PTransmission.ChangeUp();
			ModifiedInputs.GearUpInput = false;
		}

		if (ModifiedInputs.GearDownInput)
		{
			PTransmission.ChangeDown();
			ModifiedInputs.GearDownInput = false;
		}

		if (GWheeledVehicleDebugParams.ThrottleOverride > 0.f)
		{
			PTransmission.SetGear(1, true);
			ModifiedInputs.BrakeInput = 0.f;
			PEngine.SetThrottle(GWheeledVehicleDebugParams.ThrottleOverride);
		}
		else
		{
			PEngine.SetThrottle(ModifiedInputs.ThrottleInput * ModifiedInputs.ThrottleInput);
		}

		EngineBraking = PEngine.GetEngineRPM() * PEngine.Setup().EngineBrakeEffect;
	}

	for (int WheelIdx = 0; WheelIdx < PVehicle->Wheels.Num(); WheelIdx++)
	{
		auto& PWheel = PVehicle->Wheels[WheelIdx];

		float EngineBrakingForce = 0.0f;
		if ((ModifiedInputs.ThrottleInput < SMALL_NUMBER) && FMath::Abs(VehicleState.ForwardSpeed) > SMALL_NUMBER && PWheel.EngineEnabled)
		{
			EngineBrakingForce = EngineBraking;
		}

		if (PWheel.BrakeEnabled)
		{
			float BrakeForce = PWheel.MaxBrakeTorque * ModifiedInputs.BrakeInput;
			PWheel.SetBrakeTorque(TorqueMToCm(BrakeForce + EngineBrakingForce), FMath::Abs(EngineBrakingForce) > FMath::Abs(BrakeForce));
		}
		else
		{
			PWheel.SetBrakeTorque(TorqueMToCm(EngineBraking), true);
		}

		if ((ModifiedInputs.HandbrakeInput && PWheel.HandbrakeEnabled) || ModifiedInputs.ParkingEnabled)
		{
			float HandbrakeForce = ModifiedInputs.ParkingEnabled ? PWheel.HandbrakeTorque : (ModifiedInputs.HandbrakeInput * PWheel.HandbrakeTorque);
			PWheel.SetBrakeTorque(TorqueMToCm(HandbrakeForce));
		}
	}


}

bool UChaosWheeledVehicleSimulation::IsWheelSpinning() const
{
	if(!Chaos::FPhysicsSolverBase::IsNetworkPhysicsPredictionEnabled())
	{ 
		// Disable it for now since it is breaking determinism
		for (auto& Wheel : PVehicle->Wheels)
		{
			if (Wheel.IsSlipping())
			{
				return true;
			}
		}
	}

	return false;
}

void UChaosWheeledVehicleSimulation::ProcessMechanicalSimulation(float DeltaTime)
{
	using namespace Chaos;

	if (PVehicle->HasEngine())
	{
		auto& PEngine = PVehicle->GetEngine();
		auto& PTransmission = PVehicle->GetTransmission();
		auto& PDifferential = PVehicle->GetDifferential();

		float WheelRPM = 0;
		for (int I = 0; I < PVehicle->Wheels.Num(); I++)
		{
			if (PVehicle->Wheels[I].EngineEnabled)
			{
				WheelRPM = FMath::Abs(PVehicle->Wheels[I].GetWheelRPM());
			}
		}

		float WheelSpeedRPM = FMath::Abs(PTransmission.GetEngineRPMFromWheelRPM(WheelRPM));
		PEngine.SetEngineRPM(PTransmission.IsOutOfGear(), PTransmission.GetEngineRPMFromWheelRPM(WheelRPM));
		PEngine.Simulate(DeltaTime);

		PTransmission.SetEngineRPM(PEngine.GetEngineRPM()); // needs engine RPM to decide when to change gear (automatic gearbox)
		if (Chaos::FPhysicsSolverBase::IsNetworkPhysicsPredictionEnabled())
		{
			PTransmission.SetAllowedToChangeGear(true);
		}
		else
		{
			PTransmission.SetAllowedToChangeGear(!VehicleState.bVehicleInAir && !IsWheelSpinning());
		}
		float GearRatio = PTransmission.GetGearRatio(PTransmission.GetCurrentGear());

		PTransmission.Simulate(DeltaTime);

		float TransmissionTorque = PTransmission.GetTransmissionTorque(PEngine.GetEngineTorque());
		if (WheelSpeedRPM > PEngine.Setup().MaxRPM)
		{
			TransmissionTorque = 0.f;
		}

		// apply drive torque to wheels
		for (int WheelIdx = 0; WheelIdx < PVehicle->Wheels.Num(); WheelIdx++)
		{
			auto& PWheel = PVehicle->Wheels[WheelIdx];
			if (PWheel.Setup().EngineEnabled)
			{
				PWheel.SetDriveTorque(TorqueMToCm(TransmissionTorque) * PWheel.Setup().TorqueRatio);
			}
			else
			{
				PWheel.SetDriveTorque(0.f);
			}
		}

	}
}

void UChaosWheeledVehicleSimulation::DrawDebug3D()
{
	using namespace Chaos;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

	UChaosVehicleSimulation::DrawDebug3D();

	if (PVehicle == nullptr)
	{
		return;
	}

	const FTransform BodyTransform = VehicleState.VehicleWorldTransform;

	if (GWheeledVehicleDebugParams.ShowSuspensionLimits)
	{
		for (int WheelIdx = 0; WheelIdx < PVehicle->Suspension.Num(); WheelIdx++)
		{
			auto& PSuspension = PVehicle->Suspension[WheelIdx];
			auto& PWheel = PVehicle->Wheels[WheelIdx];
			// push the visualization out a bit sideways from the wheel model so we can actually see it
			FVector VehicleRightAxis = VehicleState.VehicleWorldTransform.GetUnitAxis(EAxis::Y) * 48.0f;
			const FVector& WheelOffset = PSuspension.GetLocalRestingPosition();
			if (WheelOffset.Y < 0.0f)
			{
				VehicleRightAxis = VehicleRightAxis * -1.0f;
			}

			FVector LocalDirection = PSuspension.Setup().SuspensionAxis;
			FVector WorldLocation = BodyTransform.TransformPosition(WheelOffset);
			FVector WorldDirection = BodyTransform.TransformVector(LocalDirection);

			FVector Start = WorldLocation + WorldDirection * (PWheel.GetEffectiveRadius() - PSuspension.Setup().SuspensionMaxRaise);
			FVector End = WorldLocation + WorldDirection * (PWheel.GetEffectiveRadius() + PSuspension.Setup().SuspensionMaxDrop);

			Chaos::FDebugDrawQueue::GetInstance().DrawDebugLine(
				Start + VehicleRightAxis, End + VehicleRightAxis, FColor::Orange, false, -1.f, 0, 3.f);

			FVector Start2 = WorldLocation - WorldDirection * PSuspension.Setup().SuspensionMaxRaise;
			FVector End2 = WorldLocation + WorldDirection * PSuspension.Setup().SuspensionMaxDrop;

			FDebugDrawQueue::GetInstance().DrawDebugLine(Start2 + VehicleRightAxis, End2 + VehicleRightAxis, FColor::Yellow, false, -1.f, 0, 3.f);
		}
	}

	if (GWheeledVehicleDebugParams.ShowRaycastComponent)
	{
		for (int WheelIdx = 0; WheelIdx < PVehicle->Suspension.Num(); WheelIdx++)
		{
			FHitResult& Hit = WheelState.TraceResult[WheelIdx];

			FVector VehicleRightAxis = VehicleState.VehicleWorldTransform.GetUnitAxis(EAxis::Y) * 20.0f;
			FVector VehicleUpAxis = VehicleState.VehicleWorldTransform.GetUnitAxis(EAxis::Z) * 20.0f;
			const FVector& WheelOffset = PVehicle->Suspension[WheelIdx].GetLocalRestingPosition();
			if (WheelOffset.Y < 0.0f)
			{
				VehicleRightAxis = VehicleRightAxis * -1.0f;
			}

			FVector Pt = Hit.ImpactPoint + VehicleRightAxis;
			if (Hit.GetComponent())
			{
				FDebugDrawQueue::GetInstance().DrawDebugString(Pt + VehicleRightAxis, Hit.GetComponent()->GetName(), nullptr, FColor::White, -1.f, true, 1.0f);
			}
		}
	}

	if (GWheeledVehicleDebugParams.ShowRaycastMaterial)
	{
		for (int WheelIdx = 0; WheelIdx < PVehicle->Suspension.Num(); WheelIdx++)
		{
			FHitResult& Hit = WheelState.TraceResult[WheelIdx];

			FVector VehicleRightAxis = VehicleState.VehicleWorldTransform.GetUnitAxis(EAxis::Y) * 20.0f;
			FVector VehicleUpAxis = VehicleState.VehicleWorldTransform.GetUnitAxis(EAxis::Z) * 20.0f;
			const FVector& WheelOffset = PVehicle->Suspension[WheelIdx].GetLocalRestingPosition();
			if (WheelOffset.Y < 0.0f)
			{
				VehicleRightAxis = VehicleRightAxis * -1.0f;
			}

			FVector Pt = Hit.ImpactPoint + VehicleRightAxis;
			if (Hit.PhysMaterial.IsValid())
			{
				FDebugDrawQueue::GetInstance().DrawDebugString(Pt + VehicleRightAxis + VehicleUpAxis, Hit.PhysMaterial->GetName(), nullptr, FColor::White, -1.f, true, 1.0f);
			}
		}

	}

	if (GWheeledVehicleDebugParams.ShowWheelCollisionNormal)
	{
		FString Name;
		for (int WheelIdx = 0; WheelIdx < PVehicle->Suspension.Num(); WheelIdx++)
		{
			FHitResult& Hit = WheelState.TraceResult[WheelIdx];

			FVector VehicleRightAxis = VehicleState.VehicleWorldTransform.GetUnitAxis(EAxis::Y) * 20.0f;
			const FVector& WheelOffset = PVehicle->Suspension[WheelIdx].GetLocalRestingPosition();
			if (WheelOffset.Y < 0.0f)
			{
				VehicleRightAxis = VehicleRightAxis * -1.0f;
			}

			FVector Pt = Hit.ImpactPoint + VehicleRightAxis;
			FDebugDrawQueue::GetInstance().DrawDebugLine(Pt, Pt + Hit.Normal * 20.0f, FColor::Yellow, false, 1.0f, 0, 1.0f);
			FDebugDrawQueue::GetInstance().DrawDebugSphere(Pt, 5.0f, 4, FColor::White, false, 1.0f, 0, 1.0f);
		}
	}

	if (GWheeledVehicleDebugParams.ShowSuspensionRaycasts)
	{
		for (int WheelIdx = 0; WheelIdx < PVehicle->Suspension.Num(); WheelIdx++)
		{
			const FVector& TraceStart = WheelState.Trace[WheelIdx].Start;
			const FVector& TraceEnd = WheelState.Trace[WheelIdx].End;

			// push the visualization out a bit sideways from the wheel model so we can actually see it
			FVector VehicleRightAxis = VehicleState.VehicleWorldTransform.GetUnitAxis(EAxis::Y) * 50.0f;
			const FVector& WheelOffset = PVehicle->Suspension[WheelIdx].GetLocalRestingPosition();
			if (WheelOffset.Y < 0.0f)
			{
				VehicleRightAxis = VehicleRightAxis * -1.0f;
			}

			FColor UseColor = PVehicle->Wheels[WheelIdx].InContact() ? FColor::Green : FColor::Red;
			Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(TraceStart + VehicleRightAxis, TraceEnd + VehicleRightAxis, 10.f, UseColor, false, -1.f, 0, 2.f);

			Chaos::FDebugDrawQueue::GetInstance().DrawDebugLine(TraceStart, TraceStart + VehicleRightAxis, FColor::White, false, -1.f, 0, 1.f);
			Chaos::FDebugDrawQueue::GetInstance().DrawDebugLine(TraceEnd, TraceEnd + VehicleRightAxis, FColor::White, false, -1.f, 0, 1.f);
		}
	}
#endif
}

void UChaosWheeledVehicleSimulation::FillOutputState(FChaosVehicleAsyncOutput& Output)
{
	// #Note: remember to copy/interpolate values from the physics thread output in UChaosVehicleMovementComponent::ParallelUpdate
	const auto& VehicleWheels = PVehicle->Wheels;
	auto& VehicleSuspension = PVehicle->Suspension;
	if (PVehicle->HasTransmission() && PVehicle->HasEngine())
	{
		auto& Transmission = PVehicle->GetTransmission();
		Output.VehicleSimOutput.CurrentGear = Transmission.GetCurrentGear();
		Output.VehicleSimOutput.TargetGear = Transmission.GetTargetGear();
		Output.VehicleSimOutput.TransmissionRPM = Transmission.GetTransmissionRPM();
		Output.VehicleSimOutput.TransmissionTorque = Transmission.GetTransmissionTorque(PVehicle->GetEngine().GetTorqueFromRPM(false));
	}
	if (PVehicle->HasEngine())
	{
		auto& Engine = PVehicle->GetEngine();
		Output.VehicleSimOutput.EngineRPM = Engine.GetEngineRPM();
		Output.VehicleSimOutput.EngineTorque = Engine.GetEngineTorque();
	}

	// #TODO: can we avoid copies when async is turned off
	for (int WheelIdx = 0; WheelIdx < VehicleWheels.Num(); WheelIdx++)
	{
		FWheelsOutput WheelsOut;
		WheelsOut.InContact = VehicleWheels[WheelIdx].InContact();
		WheelsOut.SteeringAngle = VehicleWheels[WheelIdx].GetSteeringAngle();
		WheelsOut.AngularPosition = VehicleWheels[WheelIdx].GetAngularPosition();
		WheelsOut.AngularVelocity = VehicleWheels[WheelIdx].GetAngularVelocity();
		WheelsOut.WheelRadius = VehicleWheels[WheelIdx].GetEffectiveRadius();

		WheelsOut.LateralAdhesiveLimit = VehicleWheels[WheelIdx].LateralAdhesiveLimit;
		WheelsOut.LongitudinalAdhesiveLimit = VehicleWheels[WheelIdx].LongitudinalAdhesiveLimit;

		WheelsOut.bIsSlipping = VehicleWheels[WheelIdx].IsSlipping();
		WheelsOut.SlipMagnitude = VehicleWheels[WheelIdx].GetSlipMagnitude();
		WheelsOut.bIsSkidding = VehicleWheels[WheelIdx].IsSkidding();
		WheelsOut.SkidMagnitude = VehicleWheels[WheelIdx].GetSkidMagnitude();
		WheelsOut.SkidNormal = WheelState.WorldWheelVelocity[WheelIdx].GetSafeNormal();
		WheelsOut.SlipAngle = VehicleWheels[WheelIdx].GetSlipAngle();

		WheelsOut.SuspensionOffset = VehicleSuspension[WheelIdx].GetSuspensionOffset();
		WheelsOut.SpringForce = VehicleSuspension[WheelIdx].GetSuspensionForce();
		WheelsOut.NormalizedSuspensionLength = VehicleSuspension[WheelIdx].GetNormalizedLength();

		WheelsOut.DriveTorque = Chaos::TorqueCmToM(VehicleWheels[WheelIdx].GetDriveTorque());
		WheelsOut.BrakeTorque = Chaos::TorqueCmToM(VehicleWheels[WheelIdx].GetBrakeTorque());

		WheelsOut.bABSActivated = VehicleWheels[WheelIdx].IsABSActivated();
		WheelsOut.bBlockingHit = WheelState.TraceResult[WheelIdx].bBlockingHit;
		WheelsOut.ImpactPoint = WheelState.TraceResult[WheelIdx].ImpactPoint;
		WheelsOut.HitLocation = WheelState.TraceResult[WheelIdx].Location;
		WheelsOut.PhysMaterial = WheelState.TraceResult[WheelIdx].PhysMaterial;

		Output.VehicleSimOutput.Wheels.Add(WheelsOut);
	}

}

UChaosWheeledVehicleSimulation::UChaosWheeledVehicleSimulation(): bOverlapHit(false)
{
	QueryBox.Init();
}

UChaosWheeledVehicleSimulation::~UChaosWheeledVehicleSimulation()
{
}

void UChaosWheeledVehicleSimulation::UpdateConstraintHandles(TArray<FPhysicsConstraintHandle>& ConstraintHandlesIn)
{
	UChaosVehicleSimulation::UpdateConstraintHandles(ConstraintHandlesIn);
	ConstraintHandles = ConstraintHandlesIn;
}

/**
 * UChaosWheeledVehicleMovementComponent
 */
UChaosWheeledVehicleMovementComponent::UChaosWheeledVehicleMovementComponent(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	// default values setup

	EngineSetup.InitDefaults();
	DifferentialSetup.InitDefaults();
	TransmissionSetup.InitDefaults();
	SteeringSetup.InitDefaults();

	// It's possible to switch whole systems off if they are not required
	bMechanicalSimEnabled = true;
	bSuspensionEnabled = true;
	bWheelFrictionEnabled = true;

	// new vehicles don't use legacy method where friction forces are applied at wheel rather than wheel contact point 
	bLegacyWheelFrictionPosition = false;

	WheelTraceCollisionResponses = FCollisionResponseContainer::GetDefaultResponseContainer();
	WheelTraceCollisionResponses.Vehicle = ECR_Ignore;
}

// Public
void UChaosWheeledVehicleMovementComponent::Serialize(FArchive & Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);

#if UE_WITH_REMOTE_OBJECT_HANDLE
	if (Ar.IsMigratingRemoteObjects())
	{
		if (Ar.IsLoading())
		{
			// Check and verify we do receive wheels properly. Then they should not be recreated in CreateVehicle()
			if (WheelSetups.Num() == Wheels.Num())
			{
				bAreWheelsMigrated = true;
			}
		}
	}
#endif
}

#if UE_WITH_REMOTE_OBJECT_HANDLE
void UChaosWheeledVehicleMovementComponent::PostMigrate(const struct FUObjectMigrationContext& MigrationContext)
{
	Super::PostMigrate(MigrationContext);

	// Set bAreWheelsMigrated back to false so it doesn't affect CreateWheels behavior. 
	bAreWheelsMigrated = false;
}
#endif


void UChaosWheeledVehicleMovementComponent::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA

	if (GetLinkerCustomVersion(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::VehicleFrictionForcePositionChange)
	{
		bLegacyWheelFrictionPosition = true;
	}

#endif  // #if WITH_EDITORONLY_DATA

}

#if WITH_EDITOR
void UChaosWheeledVehicleMovementComponent::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	RecalculateAxles();

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

void UChaosWheeledVehicleMovementComponent::FixupSkeletalMesh()
{
	Super::FixupSkeletalMesh();

	if (USkeletalMeshComponent* Mesh = Cast<USkeletalMeshComponent>(GetMesh()))
	{
		if (UPhysicsAsset* PhysicsAsset = Mesh->GetPhysicsAsset())
		{
			for (int32 WheelIdx = 0; WheelIdx < WheelSetups.Num(); ++WheelIdx)
			{
				FChaosWheelSetup& WheelSetup = WheelSetups[WheelIdx];
				if (WheelSetup.BoneName != NAME_None)
				{
					int32 BodySetupIdx = PhysicsAsset->FindBodyIndex(WheelSetup.BoneName);

					if (BodySetupIdx >= 0 && (BodySetupIdx < Mesh->Bodies.Num()))
					{
						FBodyInstance* BodyInstanceWheel = Mesh->Bodies[BodySetupIdx];
						BodyInstanceWheel->SetResponseToAllChannels(ECR_Ignore);	//turn off collision for wheel automatically

						if (UBodySetup* BodySetup = PhysicsAsset->SkeletalBodySetups[BodySetupIdx])
						{

							{
								BodyInstanceWheel->SetInstanceSimulatePhysics(false);
								//BodyInstanceWheel->SetCollisionEnabled(ECollisionEnabled::NoCollision);
							}

							bool DeleteOriginalWheelConstraints = true;
							if (DeleteOriginalWheelConstraints)
							{
								//and get rid of constraints on the wheels. TODO: right now we remove all wheel constraints, we probably only want to remove parent constraints
								TArray<int32> WheelConstraints;
								PhysicsAsset->BodyFindConstraints(BodySetupIdx, WheelConstraints);
								for (int32 ConstraintIdx = 0; ConstraintIdx < WheelConstraints.Num(); ++ConstraintIdx)
								{
									FConstraintInstance* ConInst = Mesh->Constraints[WheelConstraints[ConstraintIdx]];
									ConInst->TermConstraint();
								}
							}
						}

					}

					if (!GWheeledVehicleDebugParams.DisableConstraintSuspension)
					{
						FBodyInstance* TargetInstance = UpdatedPrimitive->GetBodyInstance();
						if (TargetInstance)
						{
							FPhysicsCommand::ExecuteWrite(TargetInstance->GetPhysicsActor(), [&](const FPhysicsActorHandle& Chassis)
								{
									const FVector LocalWheel = GetWheelRestingPosition(WheelSetup);
									FPhysicsConstraintHandle ConstraintHandle = FPhysicsInterface::CreateSuspension(Chassis, LocalWheel);

									if (ConstraintHandle.IsValid())
									{
										UChaosVehicleWheel* Wheel = Wheels[WheelIdx];
										check(Wheel);
										ConstraintHandles.Add(ConstraintHandle);
										if (Chaos::FSuspensionConstraint* Constraint = static_cast<Chaos::FSuspensionConstraint*>(ConstraintHandle.Constraint))
										{
											Constraint->SetHardstopStiffness(1.0f);
											Constraint->SetSpringStiffness(Chaos::MToCm(Wheel->SpringRate) * 0.25f);
											Constraint->SetSpringPreload(Chaos::MToCm(Wheel->SpringPreload));
											Constraint->SetSpringDamping(Wheel->SuspensionDampingRatio * 5.0f);
											Constraint->SetMinLength(-Wheel->SuspensionMaxRaise);
											Constraint->SetMaxLength(Wheel->SuspensionMaxDrop);
											Constraint->SetAxis(-Wheel->SuspensionAxis);
										}
									}
								});
						}
					}

				}
			}
		}
		VehicleSimulationPT->UpdateConstraintHandles(ConstraintHandles); // TODO: think of a better way to communicate this data

		Mesh->KinematicBonesUpdateType = EKinematicBonesUpdateToPhysics::SkipSimulatingBones;

	}

}


bool UChaosWheeledVehicleMovementComponent::CanCreateVehicle() const
{
	if (!Super::CanCreateVehicle())
		return false;

	check(GetOwner());
	FString ActorName = GetOwner()->GetName();

	for (int32 WheelIdx = 0; WheelIdx < WheelSetups.Num(); ++WheelIdx)
	{
		const FChaosWheelSetup& WheelSetup = WheelSetups[WheelIdx];

		if (WheelSetup.WheelClass == NULL)
		{
			UE_LOGF(LogVehicle, Warning, "Can't create vehicle %ls (%ls). Wheel %d is not set.", *ActorName, *GetPathName(), WheelIdx);
			return false;
		}

		if (WheelSetup.BoneName == NAME_None)
		{
			UE_LOGF(LogVehicle, Warning, "Can't create vehicle %ls (%ls). Bone name for wheel %d is not set.", *ActorName, *GetPathName(), WheelIdx);
			return false;
		}

	}

	return true;
}


void UChaosWheeledVehicleMovementComponent::OnCreatePhysicsState()
{
	Super::OnCreatePhysicsState();
}

void UChaosWheeledVehicleMovementComponent::CreateVehicle()
{
	Super::CreateVehicle();

	if (PVehicleOutput)
	{
		CreateWheels();

		// Need to bind to the notify delegate on the mesh in case physics state is changed
		if (USkeletalMeshComponent* MeshComp = GetSkeletalMesh())
		{
			MeshOnPhysicsStateChangeHandle = MeshComp->RegisterOnPhysicsCreatedDelegate(FOnSkelMeshPhysicsCreated::CreateUObject(this, &UChaosWheeledVehicleMovementComponent::RecreatePhysicsState));
			if (UVehicleAnimationInstance* VehicleAnimInstance = Cast<UVehicleAnimationInstance>(MeshComp->GetAnimInstance()))
			{
				VehicleAnimInstance->SetWheeledVehicleComponent(this);
			}
		}
	}

}

void UChaosWheeledVehicleMovementComponent::OnDestroyPhysicsState()
{
	if (PVehicleOutput.IsValid())
	{
		if (MeshOnPhysicsStateChangeHandle.IsValid())
		{
			if (USkeletalMeshComponent* MeshComp = GetSkeletalMesh())
			{
				MeshComp->UnregisterOnPhysicsCreatedDelegate(MeshOnPhysicsStateChangeHandle);
			}
		}

		DestroyWheels();

		if (ConstraintHandles.Num() > 0)
		{
			for (FPhysicsConstraintHandle ConstraintHandle : ConstraintHandles)
			{
				FPhysicsCommand::ExecuteWrite(ConstraintHandle, [&](const FPhysicsConstraintHandle& Constraint)
					{
						FPhysicsInterface::ReleaseConstraint(ConstraintHandle);
					});
			}
		}
		ConstraintHandles.Empty();
	}
	
	Super::OnDestroyPhysicsState();
	
}


void UChaosWheeledVehicleMovementComponent::NextDebugPage()
{
	int PageAsInt = (int)DebugPage;
	PageAsInt++;
	if (PageAsInt >= EDebugPages::MaxDebugPages)
	{
		PageAsInt = 0;
	}
	DebugPage = (EDebugPages)PageAsInt;
}

void UChaosWheeledVehicleMovementComponent::PrevDebugPage()
{
	int PageAsInt = (int)DebugPage;
	PageAsInt--;
	if (PageAsInt < 0)
	{
		PageAsInt = EDebugPages::MaxDebugPages - 1;
	}
	DebugPage = (EDebugPages)PageAsInt;
}


// Setup
void UChaosWheeledVehicleMovementComponent::ComputeConstants()
{
	Super::ComputeConstants();
}

void UChaosWheeledVehicleMovementComponent::CreateWheels()
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	// If this function is called within PostMigrate for a migration, the wheels should be migrated, wheel creation is avoided. 
	if (!bAreWheelsMigrated)
	{
#endif
		// Wheels num is getting copied when blueprint recompiles, so we have to manually reset here
		Wheels.Reset();

		// Instantiate the wheels
		for (int32 WheelIdx = 0; WheelIdx < WheelSetups.Num(); ++WheelIdx)
		{
			UChaosVehicleWheel* Wheel = NewObject<UChaosVehicleWheel>(this, WheelSetups[WheelIdx].WheelClass);
			check(Wheel);

			Wheels.Add(Wheel);
		}
#if UE_WITH_REMOTE_OBJECT_HANDLE
	}
#endif

	// Initialize the wheels
	for (int32 WheelIdx = 0; WheelIdx < Wheels.Num(); ++WheelIdx)
	{
		Wheels[WheelIdx]->Init(this, WheelIdx);
	}

	WheelStatus.SetNum(WheelSetups.Num());
	CachedState.SetNum(WheelSetups.Num());

	RecalculateAxles();
}

void UChaosWheeledVehicleMovementComponent::DestroyWheels()
{
	for (int32 i = 0; i < Wheels.Num(); ++i)
	{
		Wheels[i]->Shutdown();
	}

	Wheels.Reset();
}

void UChaosWheeledVehicleMovementComponent::SetupVehicle(TUniquePtr<Chaos::FSimpleWheeledVehicle>& PVehicle)
{
	using namespace Chaos;

	check(PVehicle);

	Super::SetupVehicle(PVehicle);
	NumDrivenWheels = 0;

	// we are allowed any number of wheels not limited to only 4
	for (int32 WheelIdx = 0; WheelIdx < WheelSetups.Num(); ++WheelIdx)
	{
		UChaosVehicleWheel* Wheel = WheelSetups[WheelIdx].WheelClass.GetDefaultObject();

		// create Dynamic states passing in pointer to their Static setup data
		Chaos::FSimpleWheelSim WheelSim(&Wheel->GetPhysicsWheelConfig());

		if (Wheel->GetAxleType() != EAxleType::Undefined)
		{
			bool EngineEnable = false;
			if (Wheel->GetAxleType() == EAxleType::Front)
			{
				if (DifferentialSetup.DifferentialType == EVehicleDifferential::AllWheelDrive
					|| DifferentialSetup.DifferentialType == EVehicleDifferential::FrontWheelDrive)
				{
					EngineEnable = true;
				}
			}
			else if (Wheel->GetAxleType() == EAxleType::Rear)
			{
				if (DifferentialSetup.DifferentialType == EVehicleDifferential::AllWheelDrive
					|| DifferentialSetup.DifferentialType == EVehicleDifferential::RearWheelDrive)
				{
					EngineEnable = true;
				}
			}

			WheelSim.AccessSetup().EngineEnabled = EngineEnable;
		}

		WheelSim.SetWheelRadius(Wheel->WheelRadius); // initial radius
		PVehicle->Wheels.Add(WheelSim);

		FWheelsOutput WheelsOutput; // Receptacle for Data coming out of physics simulation on physics thread
		PVehicleOutput->Wheels.Add(WheelsOutput);

		Chaos::FSimpleSuspensionSim SuspensionSim(&Wheel->GetPhysicsSuspensionConfig());
		PVehicle->Suspension.Add(SuspensionSim);

		if (WheelSim.Setup().EngineEnabled)
		{
			NumDrivenWheels++;
		}

		// for debugging to identify a single wheel
		PVehicle->Wheels[WheelIdx].SetWheelIndex(WheelIdx);
		PVehicle->Suspension[WheelIdx].SetSpringIndex(WheelIdx);
		PVehicle->NumDrivenWheels = NumDrivenWheels;

		PVehicle->bLegacyWheelFrictionPosition = bLegacyWheelFrictionPosition;
	}

	RecalculateAxles();

	// setup axles in PVehicle
	for (auto& Axle : AxleToWheelMap)
	{
		TArray<int>& WheelIndices = Axle.Value;

		FChaosWheelSetup& WheelSetup = WheelSetups[WheelIndices[0]];
		UChaosVehicleWheel* WheelData = WheelSetup.WheelClass.GetDefaultObject();

		Chaos::FAxleSim AxleSim;
		AxleSim.Setup.RollbarScaling = WheelData->RollbarScaling;

		for (int WheelIdx : WheelIndices)
		{
			AxleSim.Setup.WheelIndex.Add(WheelIdx);
		}

		PVehicle->Axles.Add(AxleSim);

	}

	// cache this value as it's useful for steering setup calculations and debug rendering
	WheelTrackDimensions = CalculateWheelLayoutDimensions();

	if (EngineSetup.TorqueCurve.GetRichCurve()->IsEmpty())
	{
		FString ActorName = "Unknown";
		if (GetOwner())
		{
			ActorName = GetOwner()->GetName();
		}
		UE_LOGF(LogVehicle, Log, "Vehicle %ls has no torque curve defined, disabling mechanical simulation.", *ActorName);

		bMechanicalSimEnabled = false;
	}

	if (bMechanicalSimEnabled)
	{
		Chaos::FSimpleEngineSim EngineSim(&EngineSetup.GetPhysicsEngineConfig());
		PVehicle->Engine.Add(EngineSim);

		Chaos::FSimpleTransmissionSim TransmissionSim(&TransmissionSetup.GetPhysicsTransmissionConfig());
		PVehicle->Transmission.Add(TransmissionSim);
		TransmissionType = TransmissionSim.Setup().TransmissionType; // current transmission mode - dynamically modifiable at runtime

		Chaos::FSimpleDifferentialSim DifferentialSim(&DifferentialSetup.GetPhysicsDifferentialConfig());
		PVehicle->Differential.Add(DifferentialSim);

		// Setup override of wheel TorqueRatio & EngineEnabled from vehicle differential settings
		for (int WheelIdx = 0; WheelIdx < PVehicle->Wheels.Num(); WheelIdx++)
		{
			FSimpleWheelSim& PWheel = PVehicle->Wheels[WheelIdx];
			bool IsWheelPowered = FTransmissionUtility::IsWheelPowered(DifferentialSim.Setup().DifferentialType, PWheel.Setup().AxleType, PWheel.EngineEnabled);
			PWheel.AccessSetup().EngineEnabled = IsWheelPowered;
			PWheel.EngineEnabled = IsWheelPowered;

			float TorqueRatio = FTransmissionUtility::GetTorqueRatioForWheel(DifferentialSim, WheelIdx, PVehicle->Wheels);
			PWheel.AccessSetup().TorqueRatio = TorqueRatio;
		}
	}

	Chaos::FSimpleSteeringSim SteeringSim(&SteeringSetup.GetPhysicsSteeringConfig(WheelTrackDimensions));
	PVehicle->Steering.Add(SteeringSim);

	Chaos::FTorqueControlSim TorqueSim(&TorqueControl.GetTorqueControlConfig());
	PVehicle->TorqueControlSim.Add(TorqueSim);

	Chaos::FTargetRotationControlSim TargetRotationSim(&TargetRotationControl.GetTargetRotationControlConfig());
	PVehicle->TargetRotationControlSim.Add(TargetRotationSim);

	Chaos::FStabilizeControlSim StabilizeSim(&StabilizeControl.GetStabilizeControlConfig());
	PVehicle->StabilizeControlSim.Add(StabilizeSim);

	// Setup the chassis and wheel shapes
	SetupVehicleShapes();

	// Setup mass properties
	SetupVehicleMass();

	// Setup Suspension
	SetupSuspension(PVehicle);
}

void UChaosWheeledVehicleMovementComponent::ResetVehicleState()
{
	UChaosVehicleMovementComponent::ResetVehicleState();

	for (FWheelStatus& WheelInfo : WheelStatus)
	{
		WheelInfo.Init();
	}
	
	for (FCachedState& State : CachedState)
	{
		State.bIsValid = false;
	}
}

void UChaosWheeledVehicleMovementComponent::SetupVehicleShapes()
{
	if (!UpdatedPrimitive)
	{
		return;
	}

}

void UChaosWheeledVehicleMovementComponent::SetupSuspension(TUniquePtr<Chaos::FSimpleWheeledVehicle>& PVehicle)
{
	if (!PVehicle.IsValid() || PVehicle->Suspension.Num() == 0)
	{
		return;
	}
	
	float TotalMass = this->Mass;
	ensureMsgf(TotalMass >= 1.0f, TEXT("The mass of this vehicle is too small."));

	TArray<FVector> LocalSpringPositions;

	// cache vehicle local position of springs
	for (int SpringIdx = 0; SpringIdx < PVehicle->Suspension.Num(); SpringIdx++)
	{
		auto& PSuspension = PVehicle->Suspension[SpringIdx];

		PSuspension.AccessSetup().MaxLength = PSuspension.Setup().SuspensionMaxDrop + PSuspension.Setup().SuspensionMaxRaise;

		FVector TotalOffset = GetWheelRestingPosition(WheelSetups[SpringIdx]);
		LocalSpringPositions.Add(TotalOffset);
		PVehicle->Suspension[SpringIdx].SetLocalRestingPosition(LocalSpringPositions[SpringIdx]);
	}

	// Calculate the mass that will rest on each of the springs
	TArray<float> OutSprungMasses;
	if (!LocalSpringPositions.IsEmpty() && !FSuspensionUtility::ComputeSprungMasses(LocalSpringPositions, TotalMass, OutSprungMasses))
	{
		// if the sprung mass calc fails fall back to something that will still simulate
		for (int Index = 0; Index < OutSprungMasses.Num(); Index++)
		{
			OutSprungMasses[Index] = TotalMass / OutSprungMasses.Num();
		}
	}

	// Calculate spring damping values we will use for physics simulation from the normalized damping ratio
	for (int SpringIdx = 0; SpringIdx < PVehicle->Suspension.Num(); SpringIdx++)
	{
		auto& Susp = PVehicle->Suspension[SpringIdx];
		float NaturalFrequency = FSuspensionUtility::ComputeNaturalFrequency(Susp.Setup().SpringRate, OutSprungMasses[SpringIdx]);
		float Damping = FSuspensionUtility::ComputeDamping(Susp.Setup().SpringRate, OutSprungMasses[SpringIdx], Susp.Setup().DampingRatio);
		UE_LOGF(LogChaos, Verbose, "Spring %d: OutNaturalFrequency %.1f Hz  (@1.0) DampingRate %.1f", SpringIdx, NaturalFrequency / (2.0f * PI), Damping);

		PVehicle->Suspension[SpringIdx].AccessSetup().ReboundDamping = Damping;
		PVehicle->Suspension[SpringIdx].AccessSetup().CompressionDamping = Damping;
		PVehicle->Suspension[SpringIdx].AccessSetup().RestingForce = OutSprungMasses[SpringIdx] * -GetGravityZ();
	}

}

void UChaosWheeledVehicleMovementComponent::RecalculateAxles()
{
	AxleToWheelMap.Empty();

	for (int WheelIdx = 0; WheelIdx < WheelSetups.Num(); WheelIdx++)
	{
		FChaosWheelSetup& WheelSetup = WheelSetups[WheelIdx];
		UChaosVehicleWheel* Wheel = WheelSetup.WheelClass.GetDefaultObject();

		if (auto* WheelIdxArray = AxleToWheelMap.Find(Wheel))
		{
			WheelIdxArray->Add(WheelIdx);
		}
		else
		{
			TArray<int> WheelIndices;
			WheelIndices.Add(WheelIdx);
			AxleToWheelMap.Add(Wheel, WheelIndices);
		}

	}
}

FVector UChaosWheeledVehicleMovementComponent::GetWheelRestingPosition(const FChaosWheelSetup& WheelSetup)
{
	FVector Offset = WheelSetup.WheelClass.GetDefaultObject()->Offset + WheelSetup.AdditionalOffset;
	return LocateBoneOffset(WheelSetup.BoneName, Offset);
}

// Access to data
float UChaosWheeledVehicleMovementComponent::GetEngineRotationSpeed() const
{
	float EngineRPM = 0.f;

	if (bMechanicalSimEnabled && PVehicleOutput)
	{
		EngineRPM = PVehicleOutput->EngineRPM;
	}

	return EngineRPM;
}

float UChaosWheeledVehicleMovementComponent::GetEngineMaxRotationSpeed() const
{
	float MaxEngineRPM = 0.f;
	
	if (bMechanicalSimEnabled)
	{
		MaxEngineRPM = EngineSetup.MaxRPM;
	}

	return MaxEngineRPM;
}

// Helper
FVector2D UChaosWheeledVehicleMovementComponent::CalculateWheelLayoutDimensions()
{
	FVector2D MaxSize(0.f, 0.f);

	for (int32 WheelIdx = 0; WheelIdx < WheelSetups.Num(); ++WheelIdx)
	{
		FChaosWheelSetup& WheelSetup = WheelSetups[WheelIdx];
		UChaosVehicleWheel* Wheel = WheelSetup.WheelClass.GetDefaultObject();
		check(Wheel);

		const FVector WheelOffset = GetWheelRestingPosition(WheelSetup);
		if (FMath::Abs(WheelOffset.Y) > MaxSize.Y)
		{
			MaxSize.Y = FMath::Abs(WheelOffset.Y);
		}

		if (FMath::Abs(WheelOffset.X) > MaxSize.X)
		{
			MaxSize.X = FMath::Abs(WheelOffset.X);
		}

	}

	// full width/length not half
	MaxSize *= 2.0f;

	return MaxSize;
}

void UChaosWheeledVehicleMovementComponent::Update(float DeltaTime)
{
	UChaosVehicleMovementComponent::Update(DeltaTime);

	if (CurAsyncInput)
	{
		if (const FBodyInstance* BodyInstance = GetBodyInstance())
		{
			if (auto Handle = BodyInstance->GetPhysicsActor())
			{
				FChaosVehicleAsyncInput* AsyncInput = static_cast<FChaosVehicleAsyncInput*>(CurAsyncInput);

				TArray<AActor*> ActorsToIgnore;
				ActorsToIgnore.Add(GetPawnOwner()); // ignore self in scene query

				FCollisionQueryParams TraceParams(NAME_None, FCollisionQueryParams::GetUnknownStatId(), false, nullptr);
				TraceParams.bReturnPhysicalMaterial = true;	// we need this to get the surface friction coefficient
				TraceParams.AddIgnoredActors(ActorsToIgnore);
				TraceParams.bTraceComplex = true;
				AsyncInput->PhysicsInputs.TraceParams = TraceParams;
				AsyncInput->PhysicsInputs.TraceCollisionResponse = WheelTraceCollisionResponses;

				AsyncInput->PhysicsInputs.WheelTraceParams.SetNum(Wheels.Num());
				for (int I = 0; I < Wheels.Num(); I++)
				{
					AsyncInput->PhysicsInputs.WheelTraceParams[I].SweepType = Wheels[I]->SweepType;
					AsyncInput->PhysicsInputs.WheelTraceParams[I].SweepShape = Wheels[I]->SweepShape;
				}
			}
		}
	}
}


// Debug
void UChaosWheeledVehicleMovementComponent::DrawDebug(UCanvas* Canvas, float& YL, float& YPos)
{
	using namespace Chaos;

	ensure(IsInGameThread());

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

	Super::DrawDebug(Canvas, YL, YPos);

	FBodyInstance* TargetInstance = GetBodyInstance();

	// #todo: is this rendering multiple times in multiplayer
	if (TargetInstance == nullptr)
	{
		return;
	}

	float ForwardSpeedMPH = CmSToMPH(GetForwardSpeed());

	// always draw this even on (DebugPage == EDebugPages::BasicPage)
	if (bMechanicalSimEnabled)
	{
		UFont* RenderFont = GEngine->GetLargeFont();
		Canvas->SetDrawColor(FColor::Yellow);

		// draw MPH, RPM and current gear
		float X, Y;
		Canvas->GetCenter(X, Y);
		float YLine = Y * 2.f - 50.f;
		float Scaling = 2.f;
		Canvas->DrawText(RenderFont, FString::Printf(TEXT("%d mph"), (int)ForwardSpeedMPH), X-100, YLine, Scaling, Scaling);
		Canvas->DrawText(RenderFont, FString::Printf(TEXT("[%d]"), (int)PVehicleOutput->CurrentGear), X, YLine, Scaling, Scaling);
		Canvas->DrawText(RenderFont, FString::Printf(TEXT("%d rpm"), (int)PVehicleOutput->EngineRPM), X+50, YLine, Scaling, Scaling);

		FVector2D DialPos(X+10, YLine-40);
		float DialRadius = 50;
		DrawDial(Canvas, DialPos, DialRadius, PVehicleOutput->EngineRPM, EngineSetup.MaxRPM);

	}

	UFont* RenderFont = GEngine->GetMediumFont();
	// draw drive data
	{
		Canvas->SetDrawColor(FColor::White);
		YPos += 16;
		
		if (bMechanicalSimEnabled)
		{
			YPos += Canvas->DrawText(RenderFont, FString::Printf(TEXT("RPM: %.1f (ChangeUp RPM %.0f, ChangeDown RPM %.0f)")
				, GetEngineRotationSpeed()
				, TransmissionSetup.ChangeUpRPM
				, TransmissionSetup.ChangeDownRPM), 4, YPos);

			YPos += Canvas->DrawText(RenderFont, FString::Printf(TEXT("Gear: %d (Target %d)")
				, GetCurrentGear(), GetTargetGear()), 4, YPos);
		}
		//YPos += Canvas->DrawText(RenderFont, FString::Printf(TEXT("Drag: %.1f"), DebugDragMagnitude), 4, YPos);

		YPos += 16;
		for (int i = 0; i < PVehicleOutput->Wheels.Num(); i++)
		{
			YPos += Canvas->DrawText(RenderFont, FString::Printf(TEXT("WheelLoad: [%d] %1.f N"), i, CmToM(WheelStatus[i].SpringForce)), 4, YPos);
		}

		YPos += 16;
		for (int i = 0; i < PVehicleOutput->Wheels.Num(); i++)
		{			
			if (WheelStatus[i].PhysMaterial.IsValid())
			{
				YPos += Canvas->DrawText(RenderFont, FString::Printf(TEXT("SurfaceFriction: [%d] %.2f"), i, WheelStatus[i].PhysMaterial->Friction), 4, YPos);
			}
		}
		
	}

	if (DebugPage == EDebugPages::PerformancePage)
	{
		if (GWheeledVehicleDebugParams.ResetPerformanceMeasurements)
		{
			GWheeledVehicleDebugParams.ResetPerformanceMeasurements = false;
			PerformanceMeasure.ResetAll();
		}

		PerformanceMeasure.Enable();

		YPos += 16;
		for (int I=0; I<PerformanceMeasure.GetNumMeasures(); I++)
		{
			const FTimeAndDistanceMeasure& Measure = PerformanceMeasure.GetMeasure(I);

			YPos += Canvas->DrawText(RenderFont, FString::Printf(TEXT("%s"), *Measure.ToString()), 4, YPos);
		}
	}

	// draw wheel layout
	if (DebugPage == EDebugPages::FrictionPage)
	{
		FVector2D MaxSize = GetWheelLayoutDimensions();

		// Draw a top down representation of the wheels in position, with the direction forces being shown
		for (int32 WheelIdx = 0; WheelIdx < WheelSetups.Num(); ++WheelIdx)
		{

			auto& PWheel = PVehicleOutput->Wheels[WheelIdx];
//			FVector Forces = PWheel.GetForceFromFriction();
//
//			FChaosWheelSetup& WheelSetup = WheelSetups[WheelIdx];
//			UChaosVehicleWheel* Wheel = WheelSetup.WheelClass.GetDefaultObject();
//			check(Wheel);
//			UPhysicalMaterial* ContactMat = Wheel->GetContactSurfaceMaterial();
//
//			const FVector WheelOffset = GetWheelRestingPosition(WheelSetup);
//
//			float DrawScale = 300;
//			FVector2D CentreDrawPosition(350, 400);
//			FVector2D WheelDrawPosition(WheelOffset.Y, -WheelOffset.X);
//			WheelDrawPosition *= DrawScale;
//			WheelDrawPosition /= MaxSize.X;
//			WheelDrawPosition += CentreDrawPosition;
//
//			FVector2D WheelDimensions(Wheel->WheelWidth, Wheel->WheelRadius * 2.0f);
//			FVector2D HalfDimensions = WheelDimensions * 0.5f;
//			FCanvasBoxItem BoxItem(WheelDrawPosition - HalfDimensions, WheelDimensions);
//			BoxItem.SetColor(FColor::Green);
//			Canvas->DrawItem(BoxItem);
//
//			float VisualScaling = 0.0001f;
//			FVector2D Force2D(Forces.Y * VisualScaling, -Forces.X * VisualScaling);
//			DrawLine2D(Canvas, WheelDrawPosition, WheelDrawPosition + Force2D, FColor::Red);
//
//			float SlipAngle = FMath::Abs(PWheel.GetSlipAngle());
//			float X = FMath::Sin(SlipAngle) * 50.f;
//			float Y = FMath::Cos(SlipAngle) * 50.f;
//
//			int Xpos = WheelDrawPosition.X + 20;
//			int Ypos = WheelDrawPosition.Y - 75.f;
//			DrawLine2D(Canvas, WheelDrawPosition, WheelDrawPosition - FVector2D(X, Y), FColor::White);
//			Ypos += Canvas->DrawText(RenderFont, FString::Printf(TEXT("Slip Angle : %d %"), (int)RadToDeg(SlipAngle)), Xpos, Ypos);
//
//			Ypos += Canvas->DrawText(RenderFont, FString::Printf(TEXT("AccelT : %.1f"), PWheel.GetDriveTorque()), Xpos, Ypos);
//			Ypos += Canvas->DrawText(RenderFont, FString::Printf(TEXT("BrakeT : %.1f"), PWheel.GetBrakeTorque()), Xpos, Ypos);
//
//			Ypos += Canvas->DrawText(RenderFont, FString::Printf(TEXT("Omega : %.2f"), PWheel.GetAngularVelocity()), Xpos, Ypos);
//
//			Ypos += Canvas->DrawText(RenderFont, FString::Printf(TEXT("GroundV : %.1f"), PWheel.GetRoadSpeed()), Xpos, Ypos);
//			Ypos += Canvas->DrawText(RenderFont, FString::Printf(TEXT("WheelV : %.1f"), PWheel.GetWheelGroundSpeed()), Xpos, Ypos);
////			Ypos += Canvas->DrawText(RenderFont, FString::Printf(TEXT("Sx : %.2f"), PWheel.GetNormalizedLongitudinalSlip()), Xpos, Ypos);
//			Ypos += Canvas->DrawText(RenderFont, FString::Printf(TEXT("Long Ad Limit : %.2f"), PWheel.LongitudinalAdhesiveLimit), Xpos, Ypos);
//			Ypos += Canvas->DrawText(RenderFont, FString::Printf(TEXT("Lat Ad Limit : %.2f"), PWheel.LateralAdhesiveLimit), Xpos, Ypos);
//			Ypos += Canvas->DrawText(RenderFont, FString::Printf(TEXT("Side Slip : %.2f"), PWheel.SideSlipModifier), Xpos, Ypos);
//
//			if (PWheel.AppliedLinearDriveForce > PWheel.LongitudinalAdhesiveLimit)
//			{
//				Canvas->SetDrawColor(FColor::Red);
//			}
//			else
//			{
//				Canvas->SetDrawColor(FColor::Green);
//			}
//			Ypos += Canvas->DrawText(RenderFont, FString::Printf(TEXT("Ap Drive : %.2f"), PWheel.AppliedLinearDriveForce), Xpos, Ypos);
//
//			if (PWheel.AppliedLinearBrakeForce > PWheel.LongitudinalAdhesiveLimit)
//			{
//				Canvas->SetDrawColor(FColor::Red);
//			}
//			else
//			{
//				Canvas->SetDrawColor(FColor::Green);
//			}
//			Ypos += Canvas->DrawText(RenderFont, FString::Printf(TEXT("Ap Brake : %.2f"), PWheel.AppliedLinearBrakeForce), Xpos, Ypos);
//			Canvas->SetDrawColor(FColor::White);
//
//			//if (PWheel.Setup().EngineEnabled)
//			//{
//			//	Ypos += Canvas->DrawText(RenderFont, FString::Printf(TEXT("RPM        : %.1f"), PWheel.GetWheelRPM()), Xpos, Ypos);
//			//	Ypos += Canvas->DrawText(RenderFont, FString::Printf(TEXT("Geared RPM : %.1f"), PTransmission.GetEngineRPMFromWheelRPM(PWheel.GetWheelRPM())), Xpos, Ypos);
//
//			//}
//
//			if (ContactMat)
//			{
//				Canvas->DrawText(RenderFont
//					, FString::Printf(TEXT("Friction %d"), ContactMat->Friction)
//					, WheelDrawPosition.X, WheelDrawPosition.Y-95.f);
//			}
		
		}

	}

	if (DebugPage == EDebugPages::SteeringPage)
	{
		//FVector2D MaxSize = GetWheelLayoutDimensions();

		//auto& PSteering = PVehicle->GetSteering();

		//FVector2D J1, J2;
		//for (int WheelIdx = 0; WheelIdx < PVehicle->Wheels.Num(); WheelIdx++)
		//{
		//	FChaosWheelSetup& WheelSetup = WheelSetups[WheelIdx];
		//	auto& PWheel = PVehicleOutput->Wheels[WheelIdx];
		//	const FVector WheelOffset = GetWheelRestingPosition(WheelSetup);

		//	float Scale = 300.0f / MaxSize.X;
		//	FVector2D CentreDrawPosition(450, 400);
		//	FVector2D WheelDrawPosition(WheelOffset.Y, -WheelOffset.X);
		//	WheelDrawPosition *= Scale;
		//	WheelDrawPosition += CentreDrawPosition;

		//	if (PWheel.Setup().SteeringEnabled)
		//	{
		//		if (WheelOffset.Y > 0)
		//		{
		//			float SteerAngle = DegToRad(PWheel.GetSteeringAngle());
		//			FVector2D Tire = FVector2D(FMath::Sin(SteerAngle), -FMath::Cos(SteerAngle)) * 30.0f;
		//			FVector2D WPt = WheelDrawPosition;
		//			DrawLine2D(Canvas, WPt - Tire, WPt + Tire, FColor::Black, 8);

		//			if (SteeringSetup.SteeringType == ESteeringType::Ackermann)
		//			{
		//				FVector2D C1, P, C2;
		//				PSteering.Ackermann.GetRightHingeLocations(C1, P, C2);
		//				C1.Y = -C1.Y;
		//				P.Y = -P.Y;
		//				C2.Y = -C2.Y;

		//				FVector2D JPt = WheelDrawPosition + (P - C2) * Scale;
		//				FVector2D CPt = WheelDrawPosition + (C1 - C2) * Scale;
		//				DrawLine2D(Canvas, CPt, JPt, FColor::Orange, 3);
		//				DrawLine2D(Canvas, WPt, JPt, FColor::Orange, 3);
		//				J1 = CPt;
		//			}
		//		}
		//		else
		//		{
		//			float SteerAngle = DegToRad(PWheel.GetSteeringAngle());
		//			FVector2D Tire = FVector2D(FMath::Sin(SteerAngle), -FMath::Cos(SteerAngle)) * 30.0f;
		//			FVector2D WPt = WheelDrawPosition;
		//			DrawLine2D(Canvas, WPt - Tire, WPt + Tire, FColor::Black, 8);

		//			if (SteeringSetup.SteeringType == ESteeringType::Ackermann)
		//			{

		//				FVector2D C1, P, C2;
		//				PSteering.Ackermann.GetLeftHingeLocations(C1, P, C2);
		//				C1.Y = -C1.Y;
		//				P.Y = -P.Y;
		//				C2.Y = -C2.Y;

		//				FVector2D JPt = WheelDrawPosition + (P - C2) * Scale;
		//				FVector2D CPt = WheelDrawPosition + (C1 - C2) * Scale;
		//				DrawLine2D(Canvas, CPt, JPt, FColor::Orange, 3);
		//				DrawLine2D(Canvas, WPt, JPt, FColor::Orange, 3);
		//				J2 = CPt;
		//			}
		//		}
		//	}
		//	else
		//	{
		//		FVector2D CPt = WheelDrawPosition;
		//		FVector2D Tire = FVector2D(0.f, 30.0f);
		//		DrawLine2D(Canvas, CPt - Tire, CPt + Tire, FColor::Black, 8);
		//	}

		//	Canvas->DrawText(RenderFont
		//		, FString::Printf(TEXT("Angle %.1f"), PWheel.GetSteeringAngle())
		//		, WheelDrawPosition.X, WheelDrawPosition.Y - 15.f);

		//}
		//DrawLine2D(Canvas, J1, J2, FColor::Red, 3);

	}

	// draw engine torque curve - just putting engine under transmission
	if (DebugPage == EDebugPages::TransmissionPage && bMechanicalSimEnabled)
	{
		auto& PTransmissionSetup = TransmissionSetup;

		float MaxTorque = EngineSetup.MaxTorque;
		int CurrentRPM = (int)PVehicleOutput->EngineRPM;
		FVector2D CurrentValue(CurrentRPM, PVehicleOutput->EngineTorque);
		int GraphWidth = 200; int GraphHeight = 120;
		int GraphXPos = 200; int GraphYPos = 400;

		Canvas->DrawDebugGraph(FString("Engine Torque Graph")
			, CurrentValue.X, CurrentValue.Y
			, GraphXPos, GraphYPos, 
			GraphWidth, GraphHeight, 
			FVector2D(0, EngineSetup.MaxRPM), FVector2D(MaxTorque, 0));

		FVector2D LastPoint;
		for (float RPM = 0; RPM <= EngineSetup.MaxRPM; RPM += 10.f)
		{
			float X = RPM / EngineSetup.MaxRPM;
			float Y = EngineSetup.GetTorqueFromRPM(RPM) / MaxTorque;
			//float Y = PEngine.GetTorqueFromRPM(RPM, false) / MaxTorque;
			FVector2D NextPoint(GraphXPos + GraphWidth * X, GraphYPos + GraphHeight - GraphHeight * Y);
			if (RPM > SMALL_NUMBER)
			{
				DrawLine2D(Canvas, LastPoint, NextPoint, FColor::Cyan);
			}
			LastPoint = NextPoint;
		}

		Canvas->DrawText(RenderFont
			, FString::Printf(TEXT("RevRate %.1f"), EngineSetup.EngineRevDownRate)
			, GraphXPos, GraphYPos);

	}

	// draw transmission torque curve
	if (DebugPage == EDebugPages::TransmissionPage && bMechanicalSimEnabled)
	{
		auto& ESetup = EngineSetup;
		auto& TSetup = TransmissionSetup;
		float MaxTorque = ESetup.MaxTorque;
		float MaxGearRatio = TSetup.ForwardGearRatios[0] * TSetup.FinalRatio; // 1st gear always has the highest multiplier
		float LongGearRatio = TSetup.ForwardGearRatios[TSetup.ForwardGearRatios.Num()-1] * TSetup.FinalRatio;
		int GraphWidth = 400; int GraphHeight = 240;
		int GraphXPos = 500; int GraphYPos = 150;

		{
			float X = PVehicleOutput->TransmissionRPM;
			float Y = PVehicleOutput->TransmissionTorque;

			FVector2D CurrentValue(X, Y);
			Canvas->DrawDebugGraph(FString("Transmission Torque Graph")
				, CurrentValue.X, CurrentValue.Y
				, GraphXPos, GraphYPos
				, GraphWidth, GraphHeight
				, FVector2D(0, ESetup.MaxRPM / LongGearRatio), FVector2D(MaxTorque* MaxGearRatio, 0));
		}

		FVector2D LastPoint;

		for (int Gear = 1; Gear <= TSetup.ForwardGearRatios.Num(); Gear++)
		{
			const float GearRatio = TSetup.GetGearRatio(Gear);
			check(GearRatio != 0.f);

			for (int EngineRPM = 0; EngineRPM <= ESetup.MaxRPM; EngineRPM += 10)
			{
				float RPMOut = EngineRPM / GearRatio;

				float X = RPMOut / (ESetup.MaxRPM / LongGearRatio);
				float Y = ESetup.GetTorqueFromRPM(EngineRPM) * TSetup.GetGearRatio(Gear) / (MaxTorque*MaxGearRatio);
				FVector2D NextPoint(GraphXPos + GraphWidth * X, GraphYPos + GraphHeight - GraphHeight * Y);
				if (EngineRPM > 0)
				{
					DrawLine2D(Canvas, LastPoint, NextPoint, FColor::Cyan);
				}
				LastPoint = NextPoint;
			}
		}
	}

	// for each of the wheel positions, draw the expected suspension movement limits and the current length
	if (DebugPage == EDebugPages::SuspensionPage)
	{
		FVector2D MaxSize = GetWheelLayoutDimensions();

		for (int32 WheelIdx = 0; WheelIdx < WheelSetups.Num(); ++WheelIdx)
		{
			FChaosWheelSetup& WheelSetup = WheelSetups[WheelIdx];
			UChaosVehicleWheel* Wheel = WheelSetup.WheelClass.GetDefaultObject();
			check(Wheel);
			UChaosVehicleWheel* VehicleWheel = Wheels[WheelIdx];

			const FVector WheelOffset = GetWheelRestingPosition(WheelSetup);

			float DrawScale = 200;
			FVector2D CentreDrawPosition(500, 350);
			FVector2D WheelDrawPosition(WheelOffset.Y, -WheelOffset.X);
			WheelDrawPosition *= DrawScale;
			WheelDrawPosition /= MaxSize.X;
			WheelDrawPosition += CentreDrawPosition;

			{
				// suspension resting position
				FVector2D Start = WheelDrawPosition + FVector2D(-10.f, 0.f);
				FVector2D End = Start + FVector2D(20.f, 0.f);
				DrawLine2D(Canvas, Start, End, FColor::Yellow, 2.f);
			}
	
			float Raise = VehicleWheel->SuspensionMaxRaise;
			float Drop = VehicleWheel->SuspensionMaxDrop;
			float Scale = 5.0f;

			{
				// suspension compression limit
				FVector2D Start = WheelDrawPosition + FVector2D(-20.f, -Raise * Scale);
				FVector2D End = Start + FVector2D(40.f, 0.f);
				DrawLine2D(Canvas, Start, End, FColor::White, 2.f);
				Canvas->DrawText(RenderFont, FString::Printf(TEXT("Raise Limit %.1f"), Raise), Start.X, Start.Y-16);
			}

			{
				// suspension extension limit
				FVector2D Start = WheelDrawPosition + FVector2D(-20.f, Drop * Scale);
				FVector2D End = Start + FVector2D(40.f, 0.f);
				DrawLine2D(Canvas, Start, End, FColor::White, 2.f);
				Canvas->DrawText(RenderFont, FString::Printf(TEXT("Drop Limit %.1f"), Drop), Start.X, Start.Y);
			}

			{
				// current suspension length
				FVector2D Start = WheelDrawPosition;
				FVector2D End = Start - FVector2D(0.f, VehicleWheel->GetSuspensionOffset() * Scale);
				DrawLine2D(Canvas, Start, End, FColor::Green, 4.f);
			}

		}
	}
#endif
}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

float UChaosWheeledVehicleMovementComponent::CalcDialAngle(float CurrentValue, float MaxValue)
{
	return (CurrentValue / MaxValue) * 3.f / 2.f * PI - (PI * 0.25f);
}

void UChaosWheeledVehicleMovementComponent::DrawDial(UCanvas* Canvas, FVector2D Pos, float Radius, float CurrentValue, float MaxValue)
{
	float Angle = CalcDialAngle(CurrentValue, MaxValue);
	FVector2D PtEnd(Pos.X - FMath::Cos(Angle) * Radius, Pos.Y - FMath::Sin(Angle) * Radius);
	DrawLine2D(Canvas, Pos, PtEnd, FColor::White, 3.f);

	for (float I = 0; I < MaxValue; I += 1000.0f)
	{
		Angle = CalcDialAngle(I, MaxValue);
		PtEnd.Set(-FMath::Cos(Angle) * Radius, -FMath::Sin(Angle) * Radius);
		FVector2D PtStart = PtEnd * 0.8f;
		DrawLine2D(Canvas, Pos + PtStart, Pos + PtEnd, FColor::White, 2.f);
	}

	// the last checkmark
	Angle = CalcDialAngle(MaxValue, MaxValue);
	PtEnd.Set(-FMath::Cos(Angle) * Radius, -FMath::Sin(Angle) * Radius);
	FVector2D PtStart = PtEnd * 0.8f;
	DrawLine2D(Canvas, Pos+PtStart, Pos+PtEnd, FColor::Red, 2.f);

}

#endif

void UChaosWheeledVehicleMovementComponent::FillWheelOutputState()
{
	if (const FChaosVehicleAsyncOutput* CurrentOutput = static_cast<FChaosVehicleAsyncOutput*>(CurAsyncOutput))
	{
		if (CurrentOutput->bValid && PVehicleOutput)
		{
			for (int WheelIdx = 0; WheelIdx < WheelStatus.Num(); WheelIdx++)
			{
				auto& PWheel = PVehicleOutput->Wheels[WheelIdx];

				FWheelStatus& State = WheelStatus[WheelIdx];

				State.bIsValid = true;
				State.bInContact = PWheel.bBlockingHit;
				State.ContactPoint = PWheel.ImpactPoint;
				State.HitLocation = PWheel.HitLocation;
				State.PhysMaterial = PWheel.PhysMaterial;
				State.NormalizedSuspensionLength = PWheel.NormalizedSuspensionLength;
				State.SpringForce = PWheel.SpringForce;
				State.SlipAngle = PWheel.SlipAngle;
				State.bIsSlipping = PWheel.bIsSlipping;
				State.SlipMagnitude = PWheel.SlipMagnitude;
				State.bIsSkidding = PWheel.bIsSkidding;
				State.SkidMagnitude = PWheel.SkidMagnitude;
				if (State.bIsSkidding)
				{
					State.SkidNormal = PWheel.SkidNormal;
					//DrawDebugLine(GetWorld()
					//	, State.ContactPoint
					//	, State.ContactPoint + State.SkidNormal
					//	, FColor::Yellow, true, -1.0f, 0, 4);
				}
				else
				{
					State.SkidNormal = FVector::ZeroVector;
				}
				State.DriveTorque = PWheel.DriveTorque;
				State.BrakeTorque = PWheel.BrakeTorque;
				State.bABSActivated = PWheel.bABSActivated;
			}
		}
	}
}


void UChaosWheeledVehicleMovementComponent::BreakWheelStatus(const struct FWheelStatus& Status, bool& bInContact, FVector& ContactPoint, UPhysicalMaterial*& PhysMaterial
	, float& NormalizedSuspensionLength, float& SpringForce, float& SlipAngle, bool& bIsSlipping, float& SlipMagnitude, bool& bIsSkidding, float& SkidMagnitude, FVector& SkidNormal, float& DriveTorque, float& BrakeTorque, bool& bABSActivated)
{
	bInContact = Status.bInContact;
	ContactPoint = Status.ContactPoint;
	PhysMaterial = Status.PhysMaterial.Get();
	NormalizedSuspensionLength = Status.NormalizedSuspensionLength;
	SpringForce = Status.SpringForce;
	SlipAngle = Status.SlipAngle;
	bIsSlipping = Status.bIsSlipping;
	SlipMagnitude = Status.SlipMagnitude;
	bIsSkidding = Status.bIsSkidding;
	SkidMagnitude = Status.SkidMagnitude;
	SkidNormal = Status.SkidNormal;
	DriveTorque = Status.DriveTorque;
	BrakeTorque = Status.BrakeTorque;
	bABSActivated = Status.bABSActivated;
}

FWheelStatus UChaosWheeledVehicleMovementComponent::MakeWheelStatus(bool bInContact, FVector& ContactPoint, UPhysicalMaterial* PhysMaterial
	, float NormalizedSuspensionLength, float SpringForce, float SlipAngle, bool bIsSlipping, float SlipMagnitude, bool bIsSkidding, float SkidMagnitude, FVector& SkidNormal, float DriveTorque, float BrakeTorque, bool bABSActivated)
{
	FWheelStatus Status;
	Status.bInContact = bInContact;
	Status.ContactPoint = ContactPoint;
	Status.PhysMaterial = PhysMaterial;
	Status.NormalizedSuspensionLength = NormalizedSuspensionLength;
	Status.SpringForce = SpringForce;
	Status.SlipAngle = SlipAngle;
	Status.bIsSlipping = bIsSlipping;
	Status.SlipMagnitude = SlipMagnitude;
	Status.bIsSkidding = bIsSkidding;
	Status.SkidMagnitude = SkidMagnitude;
	Status.SkidNormal = SkidNormal;
	Status.DriveTorque = DriveTorque;
	Status.BrakeTorque = BrakeTorque;
	Status.bABSActivated = bABSActivated;

	return Status;
}

void UChaosWheeledVehicleMovementComponent::BreakWheeledSnapshot(const struct FWheeledSnaphotData& Snapshot, FTransform& Transform, FVector& LinearVelocity
	, FVector& AngularVelocity, int& SelectedGear, float& EngineRPM, TArray<FWheelSnapshot>& WheelSnapshots)
{
	Transform = Snapshot.Transform;
	LinearVelocity = Snapshot.LinearVelocity;
	AngularVelocity = Snapshot.AngularVelocity;
	SelectedGear = Snapshot.SelectedGear;
	EngineRPM = Snapshot.EngineRPM;
	WheelSnapshots = Snapshot.WheelSnapshots;
}

FWheeledSnaphotData UChaosWheeledVehicleMovementComponent::MakeWheeledSnapshot(FTransform Transform, FVector LinearVelocity
	, FVector AngularVelocity, int SelectedGear, float EngineRPM, const TArray<FWheelSnapshot>& WheelSnapshots)
{
	FWheeledSnaphotData Snapshot;
	Snapshot.Transform = Transform;
	Snapshot.LinearVelocity = LinearVelocity;
	Snapshot.AngularVelocity = AngularVelocity;
	Snapshot.SelectedGear = SelectedGear;
	Snapshot.EngineRPM = EngineRPM;
	Snapshot.WheelSnapshots = WheelSnapshots;

	return Snapshot;
}


void UChaosWheeledVehicleMovementComponent::BreakWheelSnapshot(const struct FWheelSnapshot& Snapshot, float& SuspensionOffset
	, float& WheelRotationAngle, float& SteeringAngle, float& WheelRadius, float& WheelAngularVelocity)
{
	SuspensionOffset = Snapshot.SuspensionOffset;
	WheelRotationAngle = Snapshot.WheelRotationAngle;
	SteeringAngle = Snapshot.SteeringAngle;
	WheelRadius = Snapshot.WheelRadius;
	WheelAngularVelocity = Snapshot.WheelAngularVelocity;
}

FWheelSnapshot UChaosWheeledVehicleMovementComponent::MakeWheelSnapshot(float SuspensionOffset, float WheelRotationAngle
	, float SteeringAngle, float WheelRadius, float WheelAngularVelocity)
{
	FWheelSnapshot Snapshot;
	Snapshot.SuspensionOffset = SuspensionOffset;
	Snapshot.WheelRotationAngle = WheelRotationAngle;
	Snapshot.SteeringAngle = SteeringAngle;
	Snapshot.WheelRadius = WheelRadius;
	Snapshot.WheelAngularVelocity = WheelAngularVelocity;

	return Snapshot;
}


void UChaosWheeledVehicleMovementComponent::ParallelUpdate(float DeltaSeconds)
{
	UChaosVehicleMovementComponent::ParallelUpdate(DeltaSeconds);\

	FillWheelOutputState(); // exposes wheel/suspension data to blueprint
}

void UChaosWheeledVehicleMovementComponent::SetWheelClass(int WheelIndex, TSubclassOf<UChaosVehicleWheel> InWheelClass)
{
	if (UpdatedPrimitive && InWheelClass)
	{
		FBodyInstance* TargetInstance = GetBodyInstance();

		if (TargetInstance && WheelIndex < Wheels.Num())
		{

			FPhysicsCommand::ExecuteWrite(TargetInstance->GetPhysicsActor(), [&](const FPhysicsActorHandle& Chassis)
				{
					if (VehicleSimulationPT)
					{
						UChaosVehicleWheel* OldWheel = Wheels[WheelIndex];
						UChaosVehicleWheel* NewWheel = NewObject<UChaosVehicleWheel>(this, InWheelClass);
						NewWheel->Init(this, WheelIndex);

						VehicleSimulationPT->InitializeWheel(WheelIndex, &NewWheel->GetPhysicsWheelConfig());
						VehicleSimulationPT->InitializeSuspension(WheelIndex, &NewWheel->GetPhysicsSuspensionConfig());

						Wheels[WheelIndex] = NewWheel;

						OldWheel->Shutdown();
					}
				});			
				

		}
	}

}

FWheeledSnaphotData UChaosWheeledVehicleMovementComponent::GetSnapshot() const
{
	FWheeledSnaphotData WheelSnapshotData;
	UChaosVehicleMovementComponent::GetBaseSnapshot(WheelSnapshotData);

	WheelSnapshotData.EngineRPM = PVehicleOutput->EngineRPM;
	WheelSnapshotData.SelectedGear = PVehicleOutput->CurrentGear;
	WheelSnapshotData.WheelSnapshots.SetNum(Wheels.Num());

	int WheelIdx = 0;
	for (const UChaosVehicleWheel* Wheel : Wheels)
	{
		WheelSnapshotData.WheelSnapshots[WheelIdx].SteeringAngle = Wheel->GetSteerAngle();
		WheelSnapshotData.WheelSnapshots[WheelIdx].SuspensionOffset = Wheel->GetSuspensionOffset();
		WheelSnapshotData.WheelSnapshots[WheelIdx].WheelRotationAngle = Wheel->GetRotationAngle();
		WheelSnapshotData.WheelSnapshots[WheelIdx].WheelRadius = Wheel->GetWheelRadius();
		WheelSnapshotData.WheelSnapshots[WheelIdx].WheelAngularVelocity = Wheel->GetWheelAngularVelocity();

		WheelIdx++;
	}

	return WheelSnapshotData;
}

void UChaosWheeledVehicleMovementComponent::SetSnapshot(const FWheeledSnaphotData& SnapshotIn)
{
	UChaosVehicleMovementComponent::SetBaseSnapshot(SnapshotIn);

	FBodyInstance* TargetInstance = GetBodyInstance();
	if (TargetInstance)
	{
		FPhysicsCommand::ExecuteWrite(TargetInstance->GetPhysicsActor(), [&](const FPhysicsActorHandle& Chassis)
		{
			const FWheeledSnaphotData* WheelSnapshotData = static_cast<const FWheeledSnaphotData*>(&SnapshotIn);

			ensure(WheelSnapshotData->WheelSnapshots.Num() == VehicleSimulationPT->PVehicle->Wheels.Num());
			if (VehicleSimulationPT && VehicleSimulationPT->PVehicle)
			{
				for (int WheelIdx = 0; WheelIdx < WheelSnapshotData->WheelSnapshots.Num(); WheelIdx++)
				{
					ensure(VehicleSimulationPT->PVehicle->Wheels.Num() == VehicleSimulationPT->PVehicle->Suspension.Num());

					if (WheelIdx < VehicleSimulationPT->PVehicle->Wheels.Num())
					{
						const FWheelSnapshot& Data = WheelSnapshotData->WheelSnapshots[WheelIdx];
						Chaos::FSimpleWheelSim& VehicleWheel = VehicleSimulationPT->PVehicle->Wheels[WheelIdx];
						Chaos::FSimpleSuspensionSim& VehicleSuspension = VehicleSimulationPT->PVehicle->Suspension[WheelIdx];

						VehicleWheel.SetSteeringAngle(Data.SteeringAngle);
						VehicleSuspension.SetSuspensionLength(Data.SuspensionOffset, Data.WheelRadius);
						VehicleWheel.SetAngularPosition(FMath::DegreesToRadians(-Data.WheelRotationAngle));
						VehicleWheel.SetWheelRadius(Data.WheelRadius);
						VehicleWheel.SetAngularVelocity(Data.WheelAngularVelocity);
					}
				}
			}
		});
	}
}

//////////////////////////////////////////////////////////////////////////

void UChaosWheeledVehicleMovementComponent::SetMaxEngineTorque(float Torque)
{
	if (FBodyInstance* TargetInstance = GetBodyInstance())
	{
		FPhysicsCommand::ExecuteWrite(TargetInstance->GetPhysicsActor(), [&](const FPhysicsActorHandle& Chassis)
		{
			if (VehicleSimulationPT && VehicleSimulationPT->PVehicle && VehicleSimulationPT->PVehicle->HasEngine())
			{
				VehicleSimulationPT->PVehicle->GetEngine().SetMaxTorque(Torque);
			}
		});
	}
}

void UChaosWheeledVehicleMovementComponent::SetDragCoefficient(float DragCoeff)
{
	if (FBodyInstance* TargetInstance = GetBodyInstance())
	{
		FPhysicsCommand::ExecuteWrite(TargetInstance->GetPhysicsActor(), [&](const FPhysicsActorHandle& Chassis)
		{
			if (VehicleSimulationPT && VehicleSimulationPT->PVehicle)
			{
				VehicleSimulationPT->PVehicle->GetAerodynamics().SetDragCoefficient(DragCoeff);
			}
		});
	}
}

void UChaosWheeledVehicleMovementComponent::SetDownforceCoefficient(float DownforceCoeff)
{
	if (FBodyInstance* TargetInstance = GetBodyInstance())
	{
		FPhysicsCommand::ExecuteWrite(TargetInstance->GetPhysicsActor(), [&](const FPhysicsActorHandle& Chassis)
		{
			if (VehicleSimulationPT && VehicleSimulationPT->PVehicle)
			{
				VehicleSimulationPT->PVehicle->GetAerodynamics().SetDownforceCoefficient(DownforceCoeff);
			}
		});
	}
}

void UChaosWheeledVehicleMovementComponent::SetDifferentialFrontRearSplit(float FrontRearSplit)
{
	if (FBodyInstance* TargetInstance = GetBodyInstance())
	{
		FPhysicsCommand::ExecuteWrite(TargetInstance->GetPhysicsActor(), [&](const FPhysicsActorHandle& Chassis)
		{
			if (VehicleSimulationPT && VehicleSimulationPT->PVehicle)
			{
				VehicleSimulationPT->PVehicle->GetDifferential().FrontRearSplit = FrontRearSplit;
			}
		});
	}
}


void UChaosWheeledVehicleMovementComponent::SetTractionControlEnabled(int WheelIndex, bool Enabled)
{
	if (FBodyInstance* TargetInstance = GetBodyInstance())
	{
		FPhysicsCommand::ExecuteWrite(TargetInstance->GetPhysicsActor(), [&](const FPhysicsActorHandle& Chassis)
		{
			if (VehicleSimulationPT && VehicleSimulationPT->PVehicle && WheelIndex < VehicleSimulationPT->PVehicle->Wheels.Num())
			{
				Chaos::FSimpleWheelSim& VehicleWheel = VehicleSimulationPT->PVehicle->Wheels[WheelIndex];

				VehicleWheel.TractionControlEnabled = Enabled;
			}
		});
	}
}

void UChaosWheeledVehicleMovementComponent::SetABSEnabled(int WheelIndex, bool Enabled)
{
	if (FBodyInstance* TargetInstance = GetBodyInstance())
	{
		FPhysicsCommand::ExecuteWrite(TargetInstance->GetPhysicsActor(), [&](const FPhysicsActorHandle& Chassis)
		{
			if (VehicleSimulationPT && VehicleSimulationPT->PVehicle && WheelIndex < VehicleSimulationPT->PVehicle->Wheels.Num())
			{
				Chaos::FSimpleWheelSim& VehicleWheel = VehicleSimulationPT->PVehicle->Wheels[WheelIndex];

				VehicleWheel.ABSEnabled = Enabled;
			}
		});
	}
}

void UChaosWheeledVehicleMovementComponent::SetAffectedByBrake(int WheelIndex, bool Enabled)
{
	if (FBodyInstance* TargetInstance = GetBodyInstance())
	{
		FPhysicsCommand::ExecuteWrite(TargetInstance->GetPhysicsActor(), [&](const FPhysicsActorHandle& Chassis)
		{
			if (VehicleSimulationPT && VehicleSimulationPT->PVehicle && WheelIndex < VehicleSimulationPT->PVehicle->Wheels.Num())
			{
				Chaos::FSimpleWheelSim& VehicleWheel = VehicleSimulationPT->PVehicle->Wheels[WheelIndex];

				VehicleWheel.BrakeEnabled = Enabled;
			}
		});
	}
}

void UChaosWheeledVehicleMovementComponent::SetAffectedByHandbrake(int WheelIndex, bool Enabled)
{
	if (FBodyInstance* TargetInstance = GetBodyInstance())
	{
		FPhysicsCommand::ExecuteWrite(TargetInstance->GetPhysicsActor(), [&](const FPhysicsActorHandle& Chassis)
		{
			if (VehicleSimulationPT && VehicleSimulationPT->PVehicle && WheelIndex < VehicleSimulationPT->PVehicle->Wheels.Num())
			{
				Chaos::FSimpleWheelSim& VehicleWheel = VehicleSimulationPT->PVehicle->Wheels[WheelIndex];

				VehicleWheel.HandbrakeEnabled = Enabled; // this is affecting all vehicles - needs to be per instance 
			}
		});
	}
}

void UChaosWheeledVehicleMovementComponent::SetAffectedBySteering(int WheelIndex, bool Enabled)
{
	if (FBodyInstance* TargetInstance = GetBodyInstance())
	{
		FPhysicsCommand::ExecuteWrite(TargetInstance->GetPhysicsActor(), [&](const FPhysicsActorHandle& Chassis)
		{
			if (VehicleSimulationPT && VehicleSimulationPT->PVehicle && WheelIndex < VehicleSimulationPT->PVehicle->Wheels.Num())
			{
				Chaos::FSimpleWheelSim& VehicleWheel = VehicleSimulationPT->PVehicle->Wheels[WheelIndex];

				VehicleWheel.SteeringEnabled = Enabled;
			}
		});
	}
}

void UChaosWheeledVehicleMovementComponent::SetAffectedByEngine(int WheelIndex, bool Enabled)
{
	if (FBodyInstance* TargetInstance = GetBodyInstance())
	{
		FPhysicsCommand::ExecuteWrite(TargetInstance->GetPhysicsActor(), [&](const FPhysicsActorHandle& Chassis)
		{
			if (VehicleSimulationPT && VehicleSimulationPT->PVehicle && WheelIndex < VehicleSimulationPT->PVehicle->Wheels.Num())
			{
				Chaos::FSimpleWheelSim& VehicleWheel = VehicleSimulationPT->PVehicle->Wheels[WheelIndex];

				VehicleWheel.EngineEnabled = Enabled;
			}
		});
	}

}

void UChaosWheeledVehicleMovementComponent::SetWheelRadius(int WheelIndex, float Radius)
{
	if (FBodyInstance* TargetInstance = GetBodyInstance())
	{
		FPhysicsCommand::ExecuteWrite(TargetInstance->GetPhysicsActor(), [&](const FPhysicsActorHandle& Chassis)
		{
			if (VehicleSimulationPT && VehicleSimulationPT->PVehicle && WheelIndex < VehicleSimulationPT->PVehicle->Wheels.Num())
			{
				Chaos::FSimpleWheelSim& VehicleWheel = VehicleSimulationPT->PVehicle->Wheels[WheelIndex];

				VehicleWheel.SetWheelRadius(Radius);
			}
		});
	}
}

void UChaosWheeledVehicleMovementComponent::SetWheelFrictionMultiplier(int WheelIndex, float Friction)
{
	if (FBodyInstance* TargetInstance = GetBodyInstance())
	{
		FPhysicsCommand::ExecuteWrite(TargetInstance->GetPhysicsActor(), [&](const FPhysicsActorHandle& Chassis)
			{
				if (VehicleSimulationPT && VehicleSimulationPT->PVehicle && WheelIndex < VehicleSimulationPT->PVehicle->Wheels.Num())
				{
					Chaos::FSimpleWheelSim& VehicleWheel = VehicleSimulationPT->PVehicle->Wheels[WheelIndex];

					VehicleWheel.FrictionMultiplier = Friction;
				}
			});
	}
}

void UChaosWheeledVehicleMovementComponent::SetWheelSlipGraphMultiplier(int WheelIndex, float Multiplier)
{
	if (FBodyInstance* TargetInstance = GetBodyInstance())
	{
		FPhysicsCommand::ExecuteWrite(TargetInstance->GetPhysicsActor(), [&](const FPhysicsActorHandle& Chassis)
			{
				if (VehicleSimulationPT && VehicleSimulationPT->PVehicle && WheelIndex < VehicleSimulationPT->PVehicle->Wheels.Num())
				{
					Chaos::FSimpleWheelSim& VehicleWheel = VehicleSimulationPT->PVehicle->Wheels[WheelIndex];

					VehicleWheel.LateralSlipGraphMultiplier = Multiplier;
				}
			});
	}
}

void UChaosWheeledVehicleMovementComponent::SetWheelMaxBrakeTorque(int WheelIndex, float Torque)
{
	if (FBodyInstance* TargetInstance = GetBodyInstance())
	{
		FPhysicsCommand::ExecuteWrite(TargetInstance->GetPhysicsActor(), [&](const FPhysicsActorHandle& Chassis)
		{
			if (VehicleSimulationPT && VehicleSimulationPT->PVehicle && WheelIndex < VehicleSimulationPT->PVehicle->Wheels.Num())
			{
				Chaos::FSimpleWheelSim& VehicleWheel = VehicleSimulationPT->PVehicle->Wheels[WheelIndex];

				VehicleWheel.MaxBrakeTorque = Torque;
			}
		});
	}
}

void UChaosWheeledVehicleMovementComponent::SetWheelHandbrakeTorque(int WheelIndex, float Torque)
{
	if (FBodyInstance* TargetInstance = GetBodyInstance())
	{
		FPhysicsCommand::ExecuteWrite(TargetInstance->GetPhysicsActor(), [&](const FPhysicsActorHandle& Chassis)
		{
			if (VehicleSimulationPT && VehicleSimulationPT->PVehicle && WheelIndex < VehicleSimulationPT->PVehicle->Wheels.Num())
			{
				Chaos::FSimpleWheelSim& VehicleWheel = VehicleSimulationPT->PVehicle->Wheels[WheelIndex];

				VehicleWheel.HandbrakeTorque = Torque;
			}
		});
	}
}

void UChaosWheeledVehicleMovementComponent::SetWheelMaxSteerAngle(int WheelIndex, float AngleDegrees)
{
	if (FBodyInstance* TargetInstance = GetBodyInstance())
	{
		FPhysicsCommand::ExecuteWrite(TargetInstance->GetPhysicsActor(), [&](const FPhysicsActorHandle& Chassis)
		{
			if (VehicleSimulationPT && VehicleSimulationPT->PVehicle && WheelIndex < VehicleSimulationPT->PVehicle->Wheels.Num())
			{
				Chaos::FSimpleWheelSim& VehicleWheel = VehicleSimulationPT->PVehicle->Wheels[WheelIndex];

				VehicleWheel.MaxSteeringAngle = AngleDegrees;
			}
		});
	}
}

void UChaosWheeledVehicleMovementComponent::SetTorqueCombineMethod(ETorqueCombineMethod InCombineMethod, int32 WheelIndex)
{
	if (FBodyInstance* TargetInstance = GetBodyInstance())
	{
		FPhysicsCommand::ExecuteWrite(TargetInstance->GetPhysicsActor(), [&](const FPhysicsActorHandle& Chassis)
			{
				if (VehicleSimulationPT && VehicleSimulationPT->PVehicle && WheelIndex < VehicleSimulationPT->PVehicle->Wheels.Num())
				{
					Chaos::FSimpleWheelSim& VehicleWheel = VehicleSimulationPT->PVehicle->Wheels[WheelIndex];

					VehicleWheel.SetTorqueCombineMethod(static_cast<Chaos::FSimpleWheelConfig::EExternalTorqueCombineMethod>(InCombineMethod));
				}
			});
	}
}

void UChaosWheeledVehicleMovementComponent::SetDriveTorque(float DriveTorque, int32 WheelIndex)
{
	using namespace Chaos;

	SetSleeping(false);

	if (FBodyInstance* TargetInstance = GetBodyInstance())
	{
		FPhysicsCommand::ExecuteWrite(TargetInstance->GetPhysicsActor(), [&](const FPhysicsActorHandle& Chassis)
			{
				if (VehicleSimulationPT && VehicleSimulationPT->PVehicle && WheelIndex < VehicleSimulationPT->PVehicle->Wheels.Num())
				{
					Chaos::FSimpleWheelSim& VehicleWheel = VehicleSimulationPT->PVehicle->Wheels[WheelIndex];

					VehicleWheel.SetDriveTorqueOverride(TorqueMToCm(DriveTorque));
				}
			});
	}
}

void UChaosWheeledVehicleMovementComponent::SetBrakeTorque(float BrakeTorque, int32 WheelIndex)
{
	using namespace Chaos;

	if (FBodyInstance* TargetInstance = GetBodyInstance())
	{
		FPhysicsCommand::ExecuteWrite(TargetInstance->GetPhysicsActor(), [&](const FPhysicsActorHandle& Chassis)
			{
				if (VehicleSimulationPT && VehicleSimulationPT->PVehicle && WheelIndex < VehicleSimulationPT->PVehicle->Wheels.Num())
				{
					Chaos::FSimpleWheelSim& VehicleWheel = VehicleSimulationPT->PVehicle->Wheels[WheelIndex];

					VehicleWheel.SetBrakeTorqueOverride(TorqueMToCm(BrakeTorque));
				}
			});
	}
}

void UChaosWheeledVehicleMovementComponent::SetSuspensionParams(float Rate, float Damping, float Preload, float MaxRaise, float MaxDrop, int32 WheelIndex)
{
	using namespace Chaos;

	if (FBodyInstance* TargetInstance = GetBodyInstance())
	{
		FPhysicsCommand::ExecuteWrite(TargetInstance->GetPhysicsActor(), [&](const FPhysicsActorHandle& Chassis)
			{
				if (VehicleSimulationPT && VehicleSimulationPT->PVehicle && WheelIndex < VehicleSimulationPT->PVehicle->Wheels.Num())
				{
					Chaos::FSimpleSuspensionSim& VehicleSuspension = VehicleSimulationPT->PVehicle->Suspension[WheelIndex];

					VehicleSuspension.AccessSetup().SpringRate = Rate;
					VehicleSuspension.AccessSetup().DampingRatio = Damping;
					VehicleSuspension.AccessSetup().SpringPreload = Preload;
					VehicleSuspension.AccessSetup().SetSuspensionMaxRaise(MaxRaise);
					VehicleSuspension.AccessSetup().SetSuspensionMaxDrop(MaxDrop);
				}
			});

		FPhysicsCommand::ExecuteWrite(TargetInstance->GetPhysicsActor(), [&](const FPhysicsActorHandle& Chassis)
			{
				if (WheelIndex < WheelSetups.Num() && WheelIndex < Wheels.Num())
				{
					FChaosWheelSetup& WheelSetup = WheelSetups[WheelIndex];
					const FVector LocalWheel = GetWheelRestingPosition(WheelSetup);
					FPhysicsConstraintHandle ConstraintHandle = FPhysicsInterface::CreateSuspension(Chassis, LocalWheel);

					if (ConstraintHandle.IsValid())
					{
						UChaosVehicleWheel* Wheel = Wheels[WheelIndex];
						check(Wheel);
						if (Chaos::FSuspensionConstraint* Constraint = static_cast<Chaos::FSuspensionConstraint*>(ConstraintHandles[WheelIndex].Constraint))
						{
							Constraint->SetHardstopStiffness(1.0f);
							Constraint->SetSpringStiffness(Chaos::MToCm(Rate) * 0.25f);
							Constraint->SetSpringPreload(Chaos::MToCm(Preload));
							Constraint->SetSpringDamping(Damping * 5.0f);
							Constraint->SetMinLength(-MaxRaise);
							Constraint->SetMaxLength(MaxDrop);
						}
					}
				}
			});

	}
}

float UChaosWheeledVehicleMovementComponent::GetSuspensionOffset(int WheelIndex)
{
	float Offset = 0.f;

	auto CalcOffset = [this](ESweepShape& SweepShape, FVector& LocalHitPoint, FVector& LocalPos, float Radius)
	{
		float NewOffset = 0.0f;
		if (SweepShape == ESweepShape::Spherecast)
		{
			NewOffset = LocalHitPoint.Z - LocalPos.Z;
		}
		else
		{
			NewOffset = LocalHitPoint.Z - LocalPos.Z + Radius;
		}
		return NewOffset;
	};

	FChaosWheelSetup& WheelSetup = WheelSetups[WheelIndex];
	if (GetBodyInstance())
	{
		FTransform VehicleWorldTransform = GetBodyInstance()->GetUnrealWorldTransform();
		if (UChaosVehicleWheel* Wheel = WheelSetups[WheelIndex].WheelClass.GetDefaultObject())
		{
			if (GWheeledVehicleDebugParams.CacheSuspensionOffset && WheelStatus[WheelIndex].bIsValid)
			{
				if (WheelStatus[WheelIndex].bInContact)
				{
					float NewOffset;
					if (PVehicleOutput)
					{
						// If we have output from the async simulation, use that SuspensionOffset rather than recalculating from interpolated contact points.
						NewOffset = PVehicleOutput->Wheels[WheelIndex].SuspensionOffset;
					}
					else
					{
						FVector LocalPos = GetWheelRestingPosition(WheelSetup);
						FVector ReferencePos = (Wheel->SweepShape == ESweepShape::Spherecast) ? WheelStatus[WheelIndex].HitLocation : WheelStatus[WheelIndex].ContactPoint;
						FVector LocalHitPoint = VehicleWorldTransform.InverseTransformPosition(ReferencePos);
						NewOffset = CalcOffset(Wheel->SweepShape, LocalHitPoint, LocalPos, Wheel->WheelRadius);
					}

					if (CachedState[WheelIndex].bIsValid)
					{
						Offset = CachedState[WheelIndex].WheelOffset;

						// interpolate between old and new positions or will just jump to new position if Wheel->SuspensionSmoothing == 0
						float InterpolationMultiplier = 1.0f - (Wheel->SuspensionSmoothing / 11.0f);
						float Delta = NewOffset - CachedState[WheelIndex].WheelOffset;
						Offset += Delta * InterpolationMultiplier;
					}
					else
					{
						Offset = NewOffset;
					}
					Offset = FMath::Clamp(Offset, -Wheel->SuspensionMaxDrop, Wheel->SuspensionMaxRaise);
				}
				else
				{
					Offset = -Wheel->SuspensionMaxDrop;
				}

				CachedState[WheelIndex].bIsValid = true;
				CachedState[WheelIndex].WheelOffset = Offset;

			}
			else
			{
				if (VehicleState.bSleeping && CachedState[WheelIndex].bIsValid)
				{
					Offset = CachedState[WheelIndex].WheelOffset;
				}
				else
				{
					ECollisionChannel SpringCollisionChannel = ECollisionChannel::ECC_WorldDynamic;
					FCollisionResponseParams ResponseParams;
					ResponseParams.CollisionResponse = WheelTraceCollisionResponses;

					TArray<AActor*> ActorsToIgnore;
					ActorsToIgnore.Add(GetPawnOwner()); // ignore self in scene query

					FCollisionQueryParams TraceParams(NAME_None, FCollisionQueryParams::GetUnknownStatId(), false, nullptr);
					TraceParams.bReturnPhysicalMaterial = true;	// we need this to get the surface friction coefficient
					TraceParams.AddIgnoredActors(ActorsToIgnore);
					TraceParams.bTraceComplex = (Wheels[WheelIndex]->SweepType == ESweepType::ComplexSweep);

					FVector LocalDirection = Wheel->SuspensionAxis;
					FVector WorldLocation = VehicleWorldTransform.TransformPosition(GetWheelRestingPosition(WheelSetup));
					FVector WorldDirection = VehicleWorldTransform.TransformVector(LocalDirection);

					FVector TraceStart = WorldLocation - WorldDirection * (Wheel->SuspensionMaxRaise);
					FVector TraceEnd = WorldLocation + WorldDirection * (Wheel->SuspensionMaxDrop + Wheel->WheelRadius);

					FHitResult HitResult;
					switch (Wheel->SweepShape)
					{
						case ESweepShape::Spherecast:
						{
							float WheelRadius = Wheel->WheelRadius;

							GetWorld()->SweepSingleByChannel(HitResult
								, TraceStart
								, TraceEnd
								, FQuat::Identity, SpringCollisionChannel
								, FCollisionShape::MakeSphere(WheelRadius), TraceParams
								, ResponseParams);
						}
						break;

						case ESweepShape::Raycast:
						default:
						{
							GetWorld()->LineTraceSingleByChannel(HitResult, TraceStart, TraceEnd, SpringCollisionChannel, TraceParams, ResponseParams);
						}
						break;
					}

					if (HitResult.bBlockingHit)
					{
						FVector LocalPos = GetWheelRestingPosition(WheelSetup);
						if (Wheel->SweepShape == ESweepShape::Spherecast)
						{
							FVector LocalHitPoint = VehicleWorldTransform.InverseTransformPosition(HitResult.Location);
							Offset = CalcOffset(Wheel->SweepShape, LocalHitPoint, LocalPos, Wheel->WheelRadius);
						}
						else
						{
							FVector LocalHitPoint = VehicleWorldTransform.InverseTransformPosition(HitResult.ImpactPoint);
							Offset = CalcOffset(Wheel->SweepShape, LocalHitPoint, LocalPos, Wheel->WheelRadius);
						}

						Offset = FMath::Clamp(Offset, -Wheel->SuspensionMaxDrop, Wheel->SuspensionMaxRaise);
					}
					else
					{
						Offset = -Wheel->SuspensionMaxDrop;
					}

					CachedState[WheelIndex].bIsValid = true;
					CachedState[WheelIndex].WheelOffset = Offset;
				}
			}
		}
	}
	return Offset;
}


UPhysicalMaterial* UChaosWheeledVehicleMovementComponent::GetPhysMaterial(int WheelIndex)
{
	return WheelStatus[WheelIndex].PhysMaterial.Get();
}


FChaosWheelSetup::FChaosWheelSetup()
	: WheelClass(UChaosVehicleWheel::StaticClass())
//	, SteeringBoneName(NAME_None)
	, BoneName(NAME_None)
	, AdditionalOffset(0.0f)
{

}

FWheelState::FWheelState()
{
}

FWheelState::~FWheelState()
{
}

FWheelState::FWheelState(const FWheelState& Other): WheelLocalLocation(Other.WheelLocalLocation),
                                                    WheelWorldLocation(Other.WheelWorldLocation),
                                                    WorldWheelVelocity(Other.WorldWheelVelocity),
                                                    LocalWheelVelocity(Other.LocalWheelVelocity),
                                                    Trace(Other.Trace),
                                                    TraceResult(Other.TraceResult)
{
}

FWheelState& FWheelState::operator=(const FWheelState& Other)
{
	if (this == &Other)
		return *this;
	WheelLocalLocation = Other.WheelLocalLocation;
	WheelWorldLocation = Other.WheelWorldLocation;
	WorldWheelVelocity = Other.WorldWheelVelocity;
	LocalWheelVelocity = Other.LocalWheelVelocity;
	Trace = Other.Trace;
	TraceResult = Other.TraceResult;
	return *this;
}

void FWheelState::Init(int NumWheels)
{
	WheelLocalLocation.Init(FVector::ZeroVector, NumWheels);
	WheelWorldLocation.Init(FVector::ZeroVector, NumWheels);
	WorldWheelVelocity.Init(FVector::ZeroVector, NumWheels);
	LocalWheelVelocity.Init(FVector::ZeroVector, NumWheels);
	Trace.SetNum(NumWheels);
	TraceResult.SetNum(NumWheels);
}

#if VEHICLE_DEBUGGING_ENABLED
UE_ENABLE_OPTIMIZATION
#endif


