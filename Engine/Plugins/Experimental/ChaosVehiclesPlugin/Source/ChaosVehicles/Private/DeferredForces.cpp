// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeferredForces.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "Chaos/Particle/ParticleUtilities.h"
#include "Chaos/Utilities.h"


void FDeferredForces::AddForce(Chaos::FRigidBodyHandle_Internal* RigidHandle, const FApplyForceData& DataIn)
{
	if (ensure(RigidHandle))
	{
		Chaos::EObjectStateType ObjectState = RigidHandle->ObjectState();
		if (CHAOS_ENSURE(ObjectState == Chaos::EObjectStateType::Dynamic || ObjectState == Chaos::EObjectStateType::Sleeping))
		{
			if ((DataIn.Flags & EForceFlags::AccelChange) == EForceFlags::AccelChange)
			{
				const float RigidMass = RigidHandle->M();
				const Chaos::FVec3 Acceleration = DataIn.Force * RigidMass;
				RigidHandle->AddForce(Acceleration, false);
			}
			else
			{
				RigidHandle->AddForce(DataIn.Force, false);
			}

		}
	}

}

void FDeferredForces::AddForceAtPosition(Chaos::FRigidBodyHandle_Internal* RigidHandle, const FApplyForceAtPositionData& DataIn)
{
	if (ensure(RigidHandle))
	{
		const Chaos::FVec3 WorldCOM = Chaos::FParticleUtilitiesGT::GetCoMWorldPosition(RigidHandle);
		const Chaos::FVec3 WorldTorque = Chaos::FVec3::CrossProduct(DataIn.Position - WorldCOM, DataIn.Force);
		RigidHandle->AddForce(DataIn.Force, false);
		RigidHandle->AddTorque(WorldTorque, false);
	}
}

void FDeferredForces::AddTorque(Chaos::FRigidBodyHandle_Internal* RigidHandle, const FAddTorqueInRadiansData& DataIn)
{
	if (ensure(RigidHandle))
	{
		Chaos::EObjectStateType ObjectState = RigidHandle->ObjectState();
		if (CHAOS_ENSURE(ObjectState == Chaos::EObjectStateType::Dynamic || ObjectState == Chaos::EObjectStateType::Sleeping))
		{
			if ((DataIn.Flags & EForceFlags::AccelChange) == EForceFlags::AccelChange)
			{
				RigidHandle->AddTorque(Chaos::FParticleUtilitiesXR::GetWorldInertia(RigidHandle) * DataIn.Torque, false);
			}
			else
			{
				RigidHandle->AddTorque(DataIn.Torque, false);
			}
		}
	}
}

void FDeferredForces::AddImpulse(Chaos::FRigidBodyHandle_Internal* RigidHandle, const FAddImpulseData& DataIn)
{
	if (ensure(RigidHandle))
	{
		if ((DataIn.Flags & EForceFlags::VelChange) == EForceFlags::VelChange)
		{
			RigidHandle->SetLinearImpulse(RigidHandle->LinearImpulse() + RigidHandle->M() * DataIn.Impulse, false);
		}
		else
		{
			RigidHandle->SetLinearImpulse(RigidHandle->LinearImpulse() + DataIn.Impulse, false);
		}
	}

}

void FDeferredForces::AddImpulseAtPosition(Chaos::FRigidBodyHandle_Internal* RigidHandle, const FAddImpulseAtPositionData& DataIn)
{
	if (ensure(RigidHandle))
	{
		const Chaos::FVec3 WorldCOM = Chaos::FParticleUtilitiesGT::GetCoMWorldPosition(RigidHandle);
		const Chaos::FVec3 AngularImpulse = Chaos::FVec3::CrossProduct(DataIn.Position - WorldCOM, DataIn.Impulse);
		RigidHandle->SetLinearImpulse(RigidHandle->LinearImpulse() + DataIn.Impulse, false);
		RigidHandle->SetAngularImpulse(RigidHandle->AngularImpulse() + AngularImpulse, false);
	}
}
