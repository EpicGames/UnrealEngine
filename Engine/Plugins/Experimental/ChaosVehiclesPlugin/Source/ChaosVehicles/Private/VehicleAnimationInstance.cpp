// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UVehicleAnimationInstance.cpp: Single Node Tree Instance 
	Only plays one animation at a time. 
=============================================================================*/ 

#include "VehicleAnimationInstance.h"
#include "ChaosWheeledVehicleMovementComponent.h"
#include "WheeledVehiclePawn.h"
#include "AnimationRuntime.h"


	/////////////////////////////////////////////////////
	// UVehicleAnimationInstance
	/////////////////////////////////////////////////////

	UVehicleAnimationInstance::UVehicleAnimationInstance(const FObjectInitializer& ObjectInitializer)
		: Super(ObjectInitializer)
	{
	}

	class AWheeledVehiclePawn* UVehicleAnimationInstance::GetVehicle()
	{
		return Cast<AWheeledVehiclePawn>(GetOwningActor());
	}

	void UVehicleAnimationInstance::NativeInitializeAnimation()
	{
		// Find a wheeled movement component
		if (AActor* Actor = GetOwningActor())
		{
			if (UChaosWheeledVehicleMovementComponent* FoundWheeledVehicleComponent = Actor->FindComponentByClass<UChaosWheeledVehicleMovementComponent>())
			{
				SetWheeledVehicleComponent(FoundWheeledVehicleComponent);
			}
		}
	}

	FAnimInstanceProxy* UVehicleAnimationInstance::CreateAnimInstanceProxy()
	{
		return &AnimInstanceProxy;
	}

	void UVehicleAnimationInstance::DestroyAnimInstanceProxy(FAnimInstanceProxy* InProxy)
	{
	}

	/////////////////////////////////////////////////////
	//// PROXY ///
	/////////////////////////////////////////////////////

	void FVehicleAnimationInstanceProxy::SetWheeledVehicleComponent(const UChaosWheeledVehicleMovementComponent* InWheeledVehicleComponent)
	{
		const UChaosWheeledVehicleMovementComponent* WheeledVehicleComponent = InWheeledVehicleComponent;

		//initialize wheel data
		const int32 NumOfwheels = WheeledVehicleComponent->WheelSetups.Num();
		WheelInstances.Empty(NumOfwheels);
		if (NumOfwheels > 0)
		{
			WheelInstances.AddZeroed(NumOfwheels);
			// now add wheel data
			for (int32 WheelIndex = 0; WheelIndex < WheelInstances.Num(); ++WheelIndex)
			{
				FWheelAnimationData& WheelInstance = WheelInstances[WheelIndex];
				const FChaosWheelSetup& WheelSetup = WheeledVehicleComponent->WheelSetups[WheelIndex];

				// set data
				WheelInstance.BoneName = WheelSetup.BoneName;
				WheelInstance.LocOffset = FVector::ZeroVector;
				WheelInstance.RotOffset = FRotator::ZeroRotator;
			}
		}
	}

	void FVehicleAnimationInstanceProxy::PreUpdate(UAnimInstance* InAnimInstance, float DeltaSeconds)
	{
		Super::PreUpdate(InAnimInstance, DeltaSeconds);

		const UVehicleAnimationInstance* VehicleAnimInstance = CastChecked<UVehicleAnimationInstance>(InAnimInstance);
		if (const UChaosWheeledVehicleMovementComponent* WheeledVehicleComponent = VehicleAnimInstance->GetWheeledVehicleComponent())
		{
			for (int32 WheelIndex = 0; WheelIndex < WheelInstances.Num(); ++WheelIndex)
			{
				FWheelAnimationData& WheelInstance = WheelInstances[WheelIndex];
				if (WheeledVehicleComponent->Wheels.IsValidIndex(WheelIndex))
				{
					if (const UChaosVehicleWheel* VehicleWheel = WheeledVehicleComponent->Wheels[WheelIndex])
					{
						if (WheelSpokeCount > 0 && ShutterSpeed > 0 && MaxAngularVelocity > SMALL_NUMBER) // employ stagecoach effect
						{
							// normalized spoke transition value
							float AngularVelocity = VehicleWheel->GetRotationAngularVelocity();
							float DegreesPerFrame = AngularVelocity / ShutterSpeed;
							float DegreesPerSpoke = 360.f / WheelSpokeCount;

							float IntegerPart = 0;
							float SpokeTransition = FMath::Modf(DegreesPerFrame / DegreesPerSpoke, &IntegerPart);
							float StagecoachEffectVelocity = FMath::Sin(SpokeTransition * TWO_PI) * MaxAngularVelocity;

							// blend
							float OffsetVelocity = FMath::Abs(AngularVelocity) - MaxAngularVelocity;
							if (OffsetVelocity < 0.f)
							{
								OffsetVelocity = 0.f;
							}

							float BlendAlpha = FMath::Clamp(OffsetVelocity / MaxAngularVelocity, 0.f, 1.f);

							float CorrectedAngularVelocity = FMath::Lerp(AngularVelocity, StagecoachEffectVelocity, BlendAlpha);
							CorrectedAngularVelocity = FMath::Clamp(CorrectedAngularVelocity, -MaxAngularVelocity, MaxAngularVelocity);

							// integrate to angular position
							float RotationDelta = CorrectedAngularVelocity * DeltaSeconds;
							WheelInstance.RotOffset.Pitch += RotationDelta;

							int ExcessRotations = (int)(WheelInstance.RotOffset.Pitch / 360.0f);
							if (FMath::Abs(ExcessRotations) > 1)
							{
								WheelInstance.RotOffset.Pitch -= ExcessRotations * 360.0f;
							}
						}
						else
						{
							WheelInstance.RotOffset.Pitch = VehicleWheel->GetRotationAngle();
						}
						WheelInstance.RotOffset.Yaw = VehicleWheel->GetSteerAngle();
						WheelInstance.RotOffset.Roll = 0.f;

						WheelInstance.LocOffset = -VehicleWheel->GetSuspensionAxis() * VehicleWheel->GetSuspensionOffset();
					}
				}
			}
		}
	}

