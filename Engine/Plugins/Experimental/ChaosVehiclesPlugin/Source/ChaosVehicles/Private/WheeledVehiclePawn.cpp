// Copyright Epic Games, Inc. All Rights Reserved.

#include "WheeledVehiclePawn.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/CollisionProfile.h"
#include "ChaosVehicleMovementComponent.h"
#include "ChaosWheeledVehicleMovementComponent.h"
#include "DisplayDebugHelpers.h"

FName AWheeledVehiclePawn::VehicleMovementComponentName(TEXT("VehicleMovementComp"));
FName AWheeledVehiclePawn::VehicleMeshComponentName(TEXT("VehicleMesh"));

AWheeledVehiclePawn::AWheeledVehiclePawn(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Mesh = CreateDefaultSubobject<USkeletalMeshComponent>(VehicleMeshComponentName);
	Mesh->SetCollisionProfileName(UCollisionProfile::Vehicle_ProfileName);
	Mesh->BodyInstance.bSimulatePhysics = false;
	Mesh->BodyInstance.bNotifyRigidBodyCollision = true;
	Mesh->BodyInstance.bUseCCD = true;
	Mesh->SetGenerateOverlapEvents(true);
	Mesh->SetCanEverAffectNavigation(false);
	RootComponent = Mesh;

	VehicleMovementComponent = CreateDefaultSubobject<UChaosVehicleMovementComponent, UChaosWheeledVehicleMovementComponent>(VehicleMovementComponentName);
	VehicleMovementComponent->SetIsReplicated(true); // Enable replication by default
	VehicleMovementComponent->UpdatedComponent = Mesh;
}

void AWheeledVehiclePawn::DisplayDebug(UCanvas* Canvas, const FDebugDisplayInfo& DebugDisplay, float& YL, float& YPos)
{
	static FName NAME_Vehicle = FName(TEXT("Vehicle"));

	Super::DisplayDebug(Canvas, DebugDisplay, YL, YPos);
}

class UChaosVehicleMovementComponent* AWheeledVehiclePawn::GetVehicleMovementComponent() const
{
	return VehicleMovementComponent;
}

