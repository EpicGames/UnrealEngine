// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BoneContainer.h"
#include "BonePose.h"
#include "VehicleAnimationInstance.h"
#include "BoneControllers/AnimNode_SkeletalControlBase.h"
#include "AnimNode_WheelController.generated.h"

/**
 *	Simple controller that replaces or adds to the translation/rotation of a single bone.
 */
USTRUCT()
struct CHAOSVEHICLES_API FAnimNode_WheelController : public FAnimNode_SkeletalControlBase
{
	GENERATED_USTRUCT_BODY()

	FAnimNode_WheelController();

	// FAnimNode_Base interface
	virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface

	// FAnimNode_SkeletalControlBase interface
	virtual void EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms) override;
	virtual bool IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones) override;
	virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	// End of FAnimNode_SkeletalControlBase interface

private:
	// FAnimNode_SkeletalControlBase interface
	virtual void InitializeBoneReferences(const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface

	struct FWheelLookupData
	{
		int32 WheelIndex;
		FBoneReference BoneReference;
	};

	TArray<FWheelLookupData> Wheels;
	const FVehicleAnimationInstanceProxy* AnimInstanceProxy;	//TODO: we only cache this to use in eval where it's safe. Should change API to pass proxy into eval
};
