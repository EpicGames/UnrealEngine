// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PhysicsPublic.h"

#include "Chaos/SimCallbackInput.h"
#include "Chaos/SimCallbackObject.h"
#include "Chaos/GeometryParticlesfwd.h"
#include "ChaosVehicleManagerAsyncCallback.h"

class UChaosTireConfig;
class UChaosVehicleMovementComponent;
class FChaosScene;

class CHAOSVEHICLES_API FChaosVehicleManager
{
public:
	// Updated when vehicles need to recreate their physics state.
	// Used when values tweaked while the game is running.
	static uint32 VehicleSetupTag;

	FChaosVehicleManager(FPhysScene* PhysScene);
	~FChaosVehicleManager();

	/** Get Physics Scene */
	FPhysScene_Chaos& GetScene() const { return Scene; }

	/**
	 * Register a Physics vehicle for processing
	 */
	void AddVehicle( TWeakObjectPtr<UChaosVehicleMovementComponent> Vehicle );

	/**
	 * Unregister a Physics vehicle from processing
	 */
	void RemoveVehicle( TWeakObjectPtr<UChaosVehicleMovementComponent> Vehicle );

	/**
	 * Update vehicle tuning and other state such as input
	 */
	void ScenePreTick(FPhysScene* PhysScene, float DeltaTime);

	/** Detach this vehicle manager from a FPhysScene (remove delegates, remove from map etc) */
	void DetachFromPhysScene(FPhysScene* PhysScene);

	void Update(FPhysScene* PhysScene, float DeltaTime);
	void PostUpdate(FChaosScene* PhysScene);

	void ParallelUpdateVehicles(float DeltaSeconds);

	/** Find a vehicle manager from an FPhysScene */
	static FChaosVehicleManager* GetVehicleManagerFromScene(FPhysScene* PhysScene);

	void RegisterCallbacks();
	void UnregisterCallbacks();

private:
	/** Map of physics scenes to corresponding vehicle manager */
	static TMap<FPhysScene*, FChaosVehicleManager*> SceneToVehicleManagerMap;

	// The physics scene we belong to
	FPhysScene_Chaos& Scene;

	static bool GInitialized;

	// All instanced vehicles
	TArray<TWeakObjectPtr<UChaosVehicleMovementComponent>> Vehicles;

	FDelegateHandle OnPhysScenePreTickHandle;
	FDelegateHandle OnPhysScenePostTickHandle;

	static FDelegateHandle OnPostWorldInitializationHandle;
	static FDelegateHandle OnWorldCleanupHandle;


	FChaosVehicleManagerAsyncCallback* AsyncCallback;	// Async callback from the physics engine - we can run our simulation here
	int32 Timestamp;
	int32 SubStepCount;

	TArray<Chaos::TSimCallbackOutputHandle<FChaosVehicleManagerAsyncOutput>> PendingOutputs;
	Chaos::TSimCallbackOutputHandle<FChaosVehicleManagerAsyncOutput> LatestOutput;
};

