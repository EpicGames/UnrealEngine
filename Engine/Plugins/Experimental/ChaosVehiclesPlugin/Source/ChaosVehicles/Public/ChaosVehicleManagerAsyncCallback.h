// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PhysicsPublic.h"
#include "Chaos/SimCallbackInput.h"
#include "Chaos/SimCallbackObject.h"
#include "Chaos/GeometryParticlesfwd.h"
#include "TransmissionSystem.h"
#include "PhysicsProxy/SingleParticlePhysicsProxyFwd.h"
#include "Physics/NetworkPhysicsComponent.h"
#include "ChaosVehicleWheel.h"

#include "ChaosVehicleManagerAsyncCallback.generated.h"

class UChaosVehicleMovementComponent;

DECLARE_STATS_GROUP(TEXT("ChaosVehicleManager"), STATGROUP_ChaosVehicleManager, STATGROUP_Advanced);

enum EChaosAsyncVehicleDataType : int8
{
	AsyncInvalid,
	AsyncDefault,
};

/** Vehicle inputs from the player controller */
USTRUCT()
struct CHAOSVEHICLES_API FVehicleInputs
{
	GENERATED_USTRUCT_BODY()

		FVehicleInputs()
		: SteeringInput(0.f)
		, ThrottleInput(0.f)
		, BrakeInput(0.f)
		, PitchInput(0.f)
		, RollInput(0.f)
		, YawInput(0.f)
		, HandbrakeInput(0.f)
	{}

	// Steering output to physics system. Range -1...1
	UPROPERTY()
	float SteeringInput;

	// Accelerator output to physics system. Range 0...1
	UPROPERTY()
	float ThrottleInput;

	// Brake output to physics system. Range 0...1
	UPROPERTY()
	float BrakeInput;

	// Body Pitch output to physics system. Range -1...1
	UPROPERTY()
	float PitchInput;

	// Body Roll output to physics system. Range -1...1
	UPROPERTY()
	float RollInput;

	// Body Yaw output to physics system. Range -1...1
	UPROPERTY()
	float YawInput;

	// Handbrake output to physics system. Range 0...1
	UPROPERTY()
	float HandbrakeInput;
};

/** Control inputs holding the vehicle ones + persisten ones */
USTRUCT()
struct CHAOSVEHICLES_API FControlInputs : public FVehicleInputs
{
	GENERATED_USTRUCT_BODY()

	FControlInputs()
		: FVehicleInputs()
		, ParkingEnabled(false)
		, TransmissionType(Chaos::ETransmissionType::Automatic)
		, GearUpInput(false)
		, GearDownInput(false)
	{}

	/** Check if the parking is enabled  */
	UPROPERTY()
	bool ParkingEnabled;

	/** Type of transmission (manual/auto) */
	UPROPERTY()
	uint8 TransmissionType;

	/** Check if the user has risen the gear */
	UPROPERTY()
	bool GearUpInput;

	/** Check if the user has descreased the gear */
	UPROPERTY()
	bool GearDownInput;
};

/** Vehicle Inputs data that will be used in the inputs history to be applied while simulating */
USTRUCT()
struct CHAOSVEHICLES_API FNetworkVehicleInputs : public FNetworkPhysicsData
{
	GENERATED_USTRUCT_BODY()

	/** List of incoming control inputs coming from the local client */
	UPROPERTY()
	FControlInputs VehicleInputs;

	/** Transmission change time that could be set/changed from GT */
	UPROPERTY()
	float TransmissionChangeTime = 0.0;

	/** Transmission current gear that could be set/changed from GT */
	UPROPERTY()
	int32 TransmissionCurrentGear = 0;

	/** Transmission target gear that could be set/changed from GT */
	UPROPERTY()
	int32 TransmissionTargetGear = 0;

	/**  Apply the data onto the network physics component */
	virtual void ApplyData(UActorComponent* NetworkComponent) const override;

	/**  Build the data from the network physics component */
	virtual void BuildData(const UActorComponent* NetworkComponent) override;

	/** Decay input during resimulation */
	virtual void DecayData(float DecayAmount) override;

	/**  Serialize data function that will be used to transfer the struct across the network */
	bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess);

	/** Interpolate the data in between two inputs data */
	virtual void InterpolateData(const FNetworkPhysicsData& MinData, const FNetworkPhysicsData& MaxData) override;

	/** Merge data into this input */
	virtual void MergeData(const FNetworkPhysicsData& FromData) override;

	/** Return string with input debug data */
	virtual const FString DebugData() override;
};

template<>
struct TStructOpsTypeTraits<FNetworkVehicleInputs> : public TStructOpsTypeTraitsBase2<FNetworkVehicleInputs>
{
	enum
	{
		WithNetSerializer = true,
	};
};

/** Vehicle states data that will be used in the states history to rewind the simulation at some point inn time */
USTRUCT()
struct CHAOSVEHICLES_API FNetworkVehicleStates : public FNetworkPhysicsData
{
	GENERATED_USTRUCT_BODY()

	/** Vehicle state last velocity */
	UPROPERTY()
	FVector StateLastVelocity = FVector(0,0,0);

	/** Angular velocity for each wheels */
	UPROPERTY()
	TArray<float> WheelsOmega;

	/** Angular position for each wheels */
	UPROPERTY()
	TArray<float> WheelsAngularPosition;

	/** Suspension latest displacement to be used while simulating */
	UPROPERTY()
	TArray<float> SuspensionLastDisplacement;

	/** Suspension latest spring length to be used while simulating  */
	UPROPERTY()
	TArray<float> SuspensionLastSpringLength;

	/** Suspension averaged length for smoothing */
	UPROPERTY()
	TArray<float> SuspensionAveragedLength;

	/** Suspension averaged count for smoothing */
	UPROPERTY()
	TArray<int32> SuspensionAveragedCount;

	/** Suspension averaged number for smoothing */
	UPROPERTY()
	TArray<int32> SuspensionAveragedNum;

	/** Engine angular velocity */
	UPROPERTY()
	float EngineOmega = 0.0;

	/**  Apply the data onto the network physics component */
	virtual void ApplyData(UActorComponent* NetworkComponent) const override;

	/**  Build the data from the network physics component */
	virtual void BuildData(const UActorComponent* NetworkComponent) override;

	/**  Serialize data function that will be used to transfer the struct across the network */
	bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess);

	/** Interpolate the data in between two inputs data */
	virtual void InterpolateData(const FNetworkPhysicsData& MinData, const FNetworkPhysicsData& MaxData) override;

	/** Return string with state debug data */
	virtual const FString DebugData() override;
};

template<>
struct TStructOpsTypeTraits<FNetworkVehicleStates> : public TStructOpsTypeTraitsBase2<FNetworkVehicleStates>
{
	enum
	{
		WithNetSerializer = true,
	};
};

/** Local physics wheels outputs not replicated across the network */
USTRUCT()
struct CHAOSVEHICLES_API FWheelsOutput
{
	GENERATED_BODY()

	FWheelsOutput()
		: InContact(false)
		, SteeringAngle(0.f)
		, AngularPosition(0.f)
		, AngularVelocity(0.f)
		, WheelRadius(0.f)
		, LateralAdhesiveLimit(0.f)
		, LongitudinalAdhesiveLimit(0.f)
		, SlipAngle(0.f)
		, bIsSlipping(false)
		, SlipMagnitude(0.f)
		, bIsSkidding(false)
		, SkidMagnitude(0.f)
		, SkidNormal(FVector(1,0,0))
		, SuspensionOffset(0.f)
		, SpringForce(0.f)
		, NormalizedSuspensionLength(0.f)
		, DriveTorque(0.f)
		, BrakeTorque(0.f)
		, bABSActivated(false)
		, bBlockingHit(false)
		, ImpactPoint(FVector::ZeroVector)
		, HitLocation(FVector::ZeroVector)
		, PhysMaterial(nullptr)
	{
	}

	// wheels
	UPROPERTY()
	bool InContact;

	UPROPERTY()
	float SteeringAngle;

	UPROPERTY()
	float AngularPosition;

	UPROPERTY()
	float AngularVelocity;

	UPROPERTY()
	float WheelRadius;

	UPROPERTY()
	float LateralAdhesiveLimit;

	UPROPERTY()
	float LongitudinalAdhesiveLimit;

	UPROPERTY()
	float SlipAngle;

	UPROPERTY()
	bool bIsSlipping;

	UPROPERTY()
	float SlipMagnitude;

	UPROPERTY()
	bool bIsSkidding;

	UPROPERTY()
	float SkidMagnitude;

	UPROPERTY()
	FVector SkidNormal;

	// suspension related
	UPROPERTY()
	float SuspensionOffset;

	UPROPERTY()
	float SpringForce;

	UPROPERTY()
	float NormalizedSuspensionLength;

	UPROPERTY()
	float DriveTorque;

	UPROPERTY()
	float BrakeTorque;

	UPROPERTY()
	bool bABSActivated;

	UPROPERTY()
	bool bBlockingHit;

	UPROPERTY()
	FVector ImpactPoint;

	UPROPERTY()
	FVector HitLocation;

	UPROPERTY()
	TWeakObjectPtr<UPhysicalMaterial> PhysMaterial;
};

/**
 * Per Vehicle Output State from Physics Thread to Game Thread                            
 */
USTRUCT()
struct CHAOSVEHICLES_API FPhysicsVehicleOutput
{
	GENERATED_BODY()

	FPhysicsVehicleOutput()
		: CurrentGear(0)
		, TargetGear(0)
		, EngineRPM(0.f)
		, EngineTorque(0.f)
		, TransmissionRPM(0.f)
		, TransmissionTorque(0.f)
	{
	}

	UPROPERTY()
	TArray<FWheelsOutput> Wheels;

	UPROPERTY()
	int32 CurrentGear;

	UPROPERTY()
	int32 TargetGear;

	UPROPERTY()
	float EngineRPM;

	UPROPERTY()
	float EngineTorque;

	UPROPERTY()
	float TransmissionRPM;

	UPROPERTY()
	float TransmissionTorque;
};

struct CHAOSVEHICLES_API FWheelTraceParams
{
	ESweepType SweepType;
	ESweepShape SweepShape;
};

/**
 * Per Vehicle input State from Game Thread to Physics Thread
 */
struct CHAOSVEHICLES_API FPhysicsVehicleInputs
{
	FPhysicsVehicleInputs()
		: GravityZ(0.0f)
		, TraceParams()
		, TraceCollisionResponse()
		, WheelTraceParams()
	{
	}
	float GravityZ;
	mutable FNetworkVehicleInputs NetworkInputs;
	mutable FCollisionQueryParams TraceParams;
	mutable FCollisionResponseContainer TraceCollisionResponse;
	mutable TArray<FWheelTraceParams> WheelTraceParams;
};

struct CHAOSVEHICLES_API FPhysicsVehicleTraits
{
	using InputsType = FNetworkVehicleInputs;
	using StatesType = FNetworkVehicleStates;
};

/**
 * Per Vehicle Input State from Game Thread to Physics Thread
 */                             
struct CHAOSVEHICLES_API FChaosVehicleAsyncInput
{
	const EChaosAsyncVehicleDataType Type;
	UChaosVehicleMovementComponent* Vehicle;
	Chaos::FSingleParticlePhysicsProxy* Proxy;

	FPhysicsVehicleInputs PhysicsInputs;

	/** 
	* Vehicle simulation running on the Physics Thread
	*/
	virtual TUniquePtr<struct FChaosVehicleAsyncOutput> Simulate(UWorld* World, const float DeltaSeconds, const float TotalSeconds, bool& bWakeOut) const;

	virtual void ApplyDeferredForces(Chaos::FRigidBodyHandle_Internal* RigidHandle) const;

	FChaosVehicleAsyncInput(EChaosAsyncVehicleDataType InType = EChaosAsyncVehicleDataType::AsyncInvalid)
		: Type(InType)
		, Vehicle(nullptr)
	{
		Proxy = nullptr;	//indicates async/sync task not needed
	}

	virtual ~FChaosVehicleAsyncInput() = default;
};

struct FChaosVehicleManagerAsyncInput : public Chaos::FSimCallbackInput
{
	TArray<TUniquePtr<FChaosVehicleAsyncInput>> VehicleInputs;

	TWeakObjectPtr<UWorld> World;
	int32 Timestamp = INDEX_NONE;

	void Reset()
	{
		VehicleInputs.Reset();
		World.Reset();
	}
};

/**
 * Async Output Data
 */
struct CHAOSVEHICLES_API FChaosVehicleAsyncOutput 
{
	const EChaosAsyncVehicleDataType Type;
	bool bValid;	// indicates no work was done
	FPhysicsVehicleOutput VehicleSimOutput;

	FChaosVehicleAsyncOutput(EChaosAsyncVehicleDataType InType = EChaosAsyncVehicleDataType::AsyncInvalid)
		: Type(InType)
		, bValid(false)
	{ }

	virtual ~FChaosVehicleAsyncOutput() = default;
};

/**
 * Async Output for all of the vehicles handled by this Vehicle Manager
 */
struct FChaosVehicleManagerAsyncOutput : public Chaos::FSimCallbackOutput
{
	TArray<TUniquePtr<FChaosVehicleAsyncOutput>> VehicleOutputs;
	int32 Timestamp = INDEX_NONE;

	void Reset()
	{
		VehicleOutputs.Reset();
	}
};

/**
 * Async callback from the Physics Engine where we can perform our vehicle simulation
 */
class CHAOSVEHICLES_API FChaosVehicleManagerAsyncCallback : public Chaos::TSimCallbackObject<FChaosVehicleManagerAsyncInput, FChaosVehicleManagerAsyncOutput, Chaos::ESimCallbackOptions::Presimulate | Chaos::ESimCallbackOptions::Rewind>
{
public:
	virtual FName GetFNameForStatId() const override;
private:
	virtual void ProcessInputs_Internal(int32 PhysicsStep) override;
	virtual void OnPreSimulate_Internal() override;
};
