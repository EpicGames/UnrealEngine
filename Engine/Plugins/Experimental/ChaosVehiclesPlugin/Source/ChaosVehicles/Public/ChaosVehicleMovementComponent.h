// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "AI/Navigation/NavigationAvoidanceTypes.h"
#include "AI/RVOAvoidanceInterface.h"
#include "Curves/CurveFloat.h"
#include "GameFramework/PawnMovementComponent.h"
#include "ChaosVehicleWheel.h"
#include "AerofoilSystem.h"
#include "ThrustSystem.h"
#include "TransmissionSystem.h"
#include "PhysicsProxy/SingleParticlePhysicsProxyFwd.h"
#include "SnapshotData.h"
#include "DeferredForces.h"
#include "ChaosVehicleManagerAsyncCallback.h"
#include "CollisionQueryParams.h"
#include "RewindData.h"

#include "ChaosVehicleMovementComponent.generated.h"

class CHAOSVEHICLES_API UChaosVehicleMovementComponent;

DECLARE_LOG_CATEGORY_EXTERN(LogVehicle, Log, All);

class UCanvas;

struct FChaosVehicleAsyncInput;
struct FChaosVehicleManagerAsyncOutput;

struct CHAOSVEHICLES_API FVehicleDebugParams
{
	bool ShowCOM = false;
	bool ShowModelOrigin = false;
	bool ShowAllForces = false;
	bool ShowAerofoilForces = false;
	bool ShowAerofoilSurface = false;
	bool DisableTorqueControl = false;
	bool DisableStabilizeControl = false;
	bool DisableAerodynamics = false;
	bool DisableAerofoils = false;
	bool DisableThrusters = false;
	bool BatchQueries = false;
	bool CacheTraceOverlap = false;
	float ForceDebugScaling = 0.0006f;
	float SleepCounterThreshold = 15;
	bool DisableVehicleSleep = false;
	bool EnableMultithreading = true;
	float SetMaxMPH = 0.0f;
	float ControlInputWakeTolerance = 0.02f;
};

struct FBodyInstance;

USTRUCT()
struct CHAOSVEHICLES_API FVehicleReplicatedState : public FVehicleInputs
{
	GENERATED_USTRUCT_BODY()

	FVehicleReplicatedState() : FVehicleInputs()
	{
		TargetGear = 0;
		ThrottleUp = 0.f;
		ThrottleDown = 0.f;
	}

	// state replication: gear
	UPROPERTY()
	int32 TargetGear;

	// input replication: increase throttle
	UPROPERTY()
	float ThrottleUp;

	// input replication: decrease throttle
	UPROPERTY()
	float ThrottleDown;
};

USTRUCT()
struct CHAOSVEHICLES_API FVehicleTorqueControlConfig
{
public:
	GENERATED_USTRUCT_BODY()

		FVehicleTorqueControlConfig()
	{
		InitDefaults();
	}

	/** Torque Control Enabled */
	UPROPERTY(EditAnywhere, Category = Setup)
	bool Enabled;

	/** Yaw Torque Scaling */
	UPROPERTY(EditAnywhere, Category = Setup)
	float YawTorqueScaling;

	UPROPERTY(EditAnywhere, Category = Setup)
	float YawFromSteering;

	UPROPERTY(EditAnywhere, Category = Setup)
	float YawFromRollTorqueScaling;

	/** Pitch Torque Scaling */
	UPROPERTY(EditAnywhere, Category = Setup)
	float PitchTorqueScaling;

	/** Roll Torque Scaling */
	UPROPERTY(EditAnywhere, Category = Setup)
	float RollTorqueScaling;

	UPROPERTY(EditAnywhere, Category = Setup)
	float RollFromSteering;

	/** Rotation damping */
	UPROPERTY(EditAnywhere, Category = Setup)
	float RotationDamping;

	const Chaos::FTorqueControlConfig& GetTorqueControlConfig()
	{
		FillTorqueControlSetup();
		return PTorqueControlConfig;
	}

	void InitDefaults()
	{
		Enabled = false;
		YawTorqueScaling = 0.0f;
		YawFromSteering = 0.0f;
		YawFromRollTorqueScaling = 0.0f;
		PitchTorqueScaling = 0.0f;
		RollTorqueScaling = 0.0f;
		RollFromSteering = 0.0f;
		RotationDamping = 0.02f;
	}

private:
	void FillTorqueControlSetup()
	{
		PTorqueControlConfig.Enabled = Enabled;
		PTorqueControlConfig.YawTorqueScaling = YawTorqueScaling;
		PTorqueControlConfig.YawFromSteering = YawFromSteering;
		PTorqueControlConfig.YawFromRollTorqueScaling = YawFromRollTorqueScaling;
		PTorqueControlConfig.PitchTorqueScaling = PitchTorqueScaling;
		PTorqueControlConfig.RollTorqueScaling = RollTorqueScaling;
		PTorqueControlConfig.RollFromSteering = RollFromSteering;
		PTorqueControlConfig.RotationDamping = RotationDamping;

	}

	Chaos::FTorqueControlConfig PTorqueControlConfig;
};

USTRUCT()
struct CHAOSVEHICLES_API FVehicleTargetRotationControlConfig
{
public:
	GENERATED_USTRUCT_BODY()

	FVehicleTargetRotationControlConfig()
	{
		InitDefaults();
	}

	/** Rotation Control Enabled */
	UPROPERTY(EditAnywhere, Category = Setup)
	bool Enabled;

	UPROPERTY(EditAnywhere, Category = Setup)
	bool bRollVsSpeedEnabled;

	UPROPERTY(EditAnywhere, Category = Setup)
	float RollControlScaling;

	UPROPERTY(EditAnywhere, Category = Setup)
	float RollMaxAngle;

	UPROPERTY(EditAnywhere, Category = Setup)
	float PitchControlScaling;

	UPROPERTY(EditAnywhere, Category = Setup)
	float PitchMaxAngle;

	/** Rotation stiffness */
	UPROPERTY(EditAnywhere, Category = Setup)
	float RotationStiffness;

	/** Rotation damping */
	UPROPERTY(EditAnywhere, Category = Setup)
	float RotationDamping;

	/** Rotation mac accel */
	UPROPERTY(EditAnywhere, Category = Setup)
	float MaxAccel;

	UPROPERTY(EditAnywhere, Category = Setup)
	float AutoCentreRollStrength;

	UPROPERTY(EditAnywhere, Category = Setup)
	float AutoCentrePitchStrength;

	UPROPERTY(EditAnywhere, Category = Setup)
	float AutoCentreYawStrength;

	const Chaos::FTargetRotationControlConfig& GetTargetRotationControlConfig()
	{
		FillTargetRotationControlSetup();
		return PTargetRotationControlConfig;
	}

	void InitDefaults()
	{
		Enabled = false;

		bRollVsSpeedEnabled = false;

		RollControlScaling = 0.f;
		RollMaxAngle = 0.f;
		PitchControlScaling = 0.f;
		PitchMaxAngle = 0.f;

		RotationStiffness = 0.f;
		RotationDamping = 0.2f;
		MaxAccel = 0.f;

		AutoCentreRollStrength = 0.f;
		AutoCentrePitchStrength = 0.f;
		AutoCentreYawStrength = 0.f;
	}

private:
	void FillTargetRotationControlSetup()
	{
		PTargetRotationControlConfig.Enabled = Enabled;
		PTargetRotationControlConfig.bRollVsSpeedEnabled = bRollVsSpeedEnabled;
		PTargetRotationControlConfig.RollControlScaling = RollControlScaling;
		PTargetRotationControlConfig.RollMaxAngle = RollMaxAngle;
		PTargetRotationControlConfig.PitchControlScaling = PitchControlScaling;
		PTargetRotationControlConfig.PitchMaxAngle = PitchMaxAngle;
		PTargetRotationControlConfig.RotationStiffness = RotationStiffness;
		PTargetRotationControlConfig.RotationDamping = RotationDamping;
		PTargetRotationControlConfig.MaxAccel = MaxAccel;
		PTargetRotationControlConfig.AutoCentreRollStrength = AutoCentreRollStrength;
		PTargetRotationControlConfig.AutoCentrePitchStrength = AutoCentrePitchStrength;
		PTargetRotationControlConfig.AutoCentreYawStrength = AutoCentreYawStrength;
	}

	Chaos::FTargetRotationControlConfig PTargetRotationControlConfig;
};

USTRUCT()
struct CHAOSVEHICLES_API FVehicleStabilizeControlConfig
{
public:
	GENERATED_USTRUCT_BODY()

	FVehicleStabilizeControlConfig()
	{
		InitDefaults();
	}

	/** Torque Control Enabled */
	UPROPERTY(EditAnywhere, Category = Setup)
	bool Enabled;

	/** Yaw Torque Scaling */
	UPROPERTY(EditAnywhere, Category = Setup)
	float AltitudeHoldZ;

	UPROPERTY(EditAnywhere, Category = Setup)
	float PositionHoldXY;

	const Chaos::FStabilizeControlConfig& GetStabilizeControlConfig()
	{
		FillStabilizeControlSetup();
		return PStabilizeControlConfig;
	}

	void InitDefaults()
	{
		Enabled = false;
		AltitudeHoldZ = 4.0f;
		PositionHoldXY = 8.0f;
	}

private:
	void FillStabilizeControlSetup()
	{
		PStabilizeControlConfig.Enabled = this->Enabled;
		PStabilizeControlConfig.AltitudeHoldZ = this->AltitudeHoldZ;
		PStabilizeControlConfig.PositionHoldXY = this->PositionHoldXY;
	}

	Chaos::FStabilizeControlConfig PStabilizeControlConfig;

};

/** Commonly used state - evaluated once used wherever required */
struct CHAOSVEHICLES_API FVehicleState
{
	FVehicleState()
		: VehicleWorldTransform(FTransform::Identity)
		, VehicleWorldVelocity(FVector::ZeroVector)
		, VehicleLocalVelocity(FVector::ZeroVector)
		, VehicleWorldAngularVelocity(FVector::ZeroVector)
		, VehicleWorldCOM(FVector::ZeroVector)
		, WorldVelocityNormal(FVector::ZeroVector)
		, VehicleUpAxis(FVector(0.f, 0.f, 1.f))
		, VehicleForwardAxis(FVector(1.f, 0.f, 0.f))
		, VehicleRightAxis(FVector(0.f, 1.f, 0.f))
		, LocalAcceleration(FVector::ZeroVector)
		, LocalGForce(FVector::ZeroVector)
		, LastFrameVehicleLocalVelocity(FVector::ZeroVector)
		, ForwardSpeed(0.f)
		, ForwardsAcceleration(0.f)
		, NumWheelsOnGround(0)
		, bAllWheelsOnGround(false)
		, bVehicleInAir(true)
		, bSleeping(false)
		, SleepCounter(0)
	{

	}

	/** Cache some useful data at the start of the frame from GT BodyInstance */
	void CaptureState(const FBodyInstance* TargetInstance, float GravityZ, float DeltaTime);

	/** Cache some useful data at the start of the frame from Physics thread Particle Handle */
	void CaptureState(const Chaos::FRigidBodyHandle_Internal* Handle, float GravityZ, float DeltaTime);

	FTransform VehicleWorldTransform;
	FVector VehicleWorldVelocity;
	FVector VehicleLocalVelocity;
	FVector VehicleWorldAngularVelocity;
	FVector VehicleWorldCOM;
	FVector WorldVelocityNormal;

	FVector VehicleUpAxis;
	FVector VehicleForwardAxis;
	FVector VehicleRightAxis;
	FVector LocalAcceleration;
	FVector LocalGForce;
	FVector LastFrameVehicleLocalVelocity;

	float ForwardSpeed;
	float ForwardsAcceleration;

	int NumWheelsOnGround;
	bool bAllWheelsOnGround;
	bool bVehicleInAir;
	bool bSleeping;
	int SleepCounter;
};

/** Input Options */
UENUM()
enum class EInputFunctionType : uint8
{
	LinearFunction = 0,
	SquaredFunction,
	CustomCurve
};

USTRUCT()
struct CHAOSVEHICLES_API FVehicleInputRateConfig
{
	GENERATED_USTRUCT_BODY()

	/**
	 * Rate at which the input value rises
	 */
	UPROPERTY(EditAnywhere, Category=VehicleInputRate)
	float RiseRate;

	/**
	 * Rate at which the input value falls
	 */
	UPROPERTY(EditAnywhere, Category=VehicleInputRate)
	float FallRate;

	/**
	 * Controller input curve, various predefined options, linear, squared, or user can specify a custom curve function
	 */
	UPROPERTY(EditAnywhere, Category=VehicleInputRate)
	EInputFunctionType InputCurveFunction;

	/**
	 * Controller input curve - should be a normalized float curve, i.e. time from 0 to 1 and values between 0 and 1
	 * This curve is only sued if the InputCurveFunction above is set to CustomCurve
	 */
	UPROPERTY(EditAnywhere, Category = VehicleInputRate)
	FRuntimeFloatCurve UserCurve;

	FVehicleInputRateConfig() : RiseRate(5.0f), FallRate(5.0f), InputCurveFunction(EInputFunctionType::LinearFunction) { }

	/** Change an output value using max rise and fall rates */
	float InterpInputValue(float DeltaTime, float CurrentValue, float NewValue) const
	{
		const float DeltaValue = NewValue - CurrentValue;

		// We are "rising" when DeltaValue has the same sign as CurrentValue (i.e. delta causes an absolute magnitude gain)
		// OR we were at 0 before, and our delta is no longer 0.
		const bool bRising = ((DeltaValue > 0.0f) == (CurrentValue > 0.0f)) ||
			((DeltaValue != 0.f) && (CurrentValue == 0.f));

		const float MaxDeltaValue = DeltaTime * (bRising ? RiseRate : FallRate);
		const float ClampedDeltaValue = FMath::Clamp(DeltaValue, -MaxDeltaValue, MaxDeltaValue);
		return CurrentValue + ClampedDeltaValue;
	}

	float CalcControlFunction(float InputValue)
	{
		// user defined curve

		// else use option from drop down list
		switch (InputCurveFunction)
		{
		case EInputFunctionType::CustomCurve:
		{
			if (UserCurve.GetRichCurveConst() && !UserCurve.GetRichCurveConst()->IsEmpty())
			{
				float Output = FMath::Clamp(UserCurve.GetRichCurveConst()->Eval(FMath::Abs(InputValue)), 0.0f, 1.0f);
				return (InputValue < 0.f) ? -Output : Output;
			}
			else
			{
				return InputValue;
			}
		}
		break;
		case EInputFunctionType::SquaredFunction:
		{
			return (InputValue < 0.f) ? -InputValue * InputValue : InputValue * InputValue;
		}
		break;

		case EInputFunctionType::LinearFunction:
		default:
		{
			return InputValue;
		}
		break;

		}

	}
};


UENUM()
enum class EVehicleAerofoilType : uint8
{
	Fixed = 0,
	Wing,			// affected by Roll input
	Rudder,			// affected by steering/yaw input
	Elevator		// affected by Pitch input
};


UENUM()
enum class EVehicleThrustType : uint8
{
	Fixed = 0,
	Wing,				// affected by Roll input
	Rudder,				// affected by steering/yaw input
	Elevator,			// affected by Pitch input
//	HelicopterRotor,	// affected by pitch/roll inputs
};


USTRUCT()
struct CHAOSVEHICLES_API FVehicleAerofoilConfig
{
	GENERATED_USTRUCT_BODY()

	FVehicleAerofoilConfig()
	{
		InitDefaults();
	}

	// Does this aerofoil represent a fixed spoiler, an aircraft wing, etc how is controlled.
	UPROPERTY(EditAnywhere, Category = AerofoilSetup)
	EVehicleAerofoilType AerofoilType;

	// Bone name on mesh where aerofoil is centered
	UPROPERTY(EditAnywhere, Category = AerofoilSetup)
	FName BoneName;

	// Additional offset to give the aerofoil.
	UPROPERTY(EditAnywhere, Category = AerofoilSetup)
	FVector Offset;

	// Up Axis of aerofoil.
	UPROPERTY(EditAnywhere, Category = AerofoilSetup)
	FVector UpAxis;

	// Area of aerofoil surface [Meters Squared] - larger value creates more lift but also more drag
	UPROPERTY(EditAnywhere, Category = AerofoilSetup)
	float Area;

	// camber of wing - leave as zero for a rudder - can be used to trim/level elevator for level flight
	UPROPERTY(EditAnywhere, Category = AerofoilSetup)
	float Camber;

	// The angle in degrees through which the control surface moves - leave at 0 if it is a fixed surface
	UPROPERTY(EditAnywhere, Category = AerofoilSetup)
	float MaxControlAngle;

	UPROPERTY(EditAnywhere, Category = AerofoilSetup)
	float StallAngle;

	// cheat to control amount of lift independently from lift
	UPROPERTY(EditAnywhere, Category = AerofoilSetup)
	float LiftMultiplier;

	// cheat to control amount of drag independently from lift, a value of zero will offer no drag
	UPROPERTY(EditAnywhere, Category = AerofoilSetup)
	float DragMultiplier;

	const Chaos::FAerofoilConfig& GetPhysicsAerofoilConfig(const UChaosVehicleMovementComponent& MovementComponent)
	{
		FillAerofoilSetup(MovementComponent);
		return PAerofoilConfig;
	}

	void InitDefaults()
	{
		AerofoilType = EVehicleAerofoilType::Fixed;
		BoneName = NAME_None;
		Offset = FVector::ZeroVector;
		UpAxis = FVector(0.f, 0.f, -1.f);
		Area = 1.f;
		Camber = 3.f;
		MaxControlAngle = 0.f;
		StallAngle = 16.f;
		LiftMultiplier = 1.0f;
		DragMultiplier = 1.0f;
	}

private:

	void FillAerofoilSetup(const UChaosVehicleMovementComponent& MovementComponent);

	Chaos::FAerofoilConfig PAerofoilConfig;
};

USTRUCT()
struct CHAOSVEHICLES_API FVehicleThrustConfig
{
	GENERATED_USTRUCT_BODY()

	FVehicleThrustConfig()
	{
		InitDefaults();
	}

	// Does this aerofoil represent a fixed spoiler, an aircraft wing, etc how is controlled.
	UPROPERTY(EditAnywhere, Category = ThrustSetup)
	EVehicleThrustType ThrustType;

	/** Bone name on mesh where thrust is located */
	UPROPERTY(EditAnywhere, Category = ThrustSetup)
	FName BoneName;

	/** Additional offset to give the location, or use in preference to the bone */
	UPROPERTY(EditAnywhere, Category = ThrustSetup)
	FVector Offset;

	/** Up Axis of thrust. */
	UPROPERTY(EditAnywhere, Category = ThrustSetup)
	FVector ThrustAxis;

	///** How the thrust is applied as the speed increases */
	//UPROPERTY(EditAnywhere, Category = ThrustSetup)
	//FRuntimeFloatCurve ThrustCurve;

	///** Maximum speed after which the thrust will cut off */
	//UPROPERTY(EditAnywhere, Category = ThrustSetup)
	//float MaxSpeed;

	/** Maximum thrust force */
	UPROPERTY(EditAnywhere, Category = ThrustSetup)
	float MaxThrustForce;

	/** The angle in degrees through which the control surface moves - leave at 0 if it is a fixed surface */
	UPROPERTY(EditAnywhere, Category = ThrustSetup)
	float MaxControlAngle;

	// #todo:ControlAxes - X, Y, Z, or X & Y, etc
	const Chaos::FSimpleThrustConfig& GetPhysicsThrusterConfig(const UChaosVehicleMovementComponent& MovementComponent)
	{
		FillThrusterSetup(MovementComponent);
		return PThrusterConfig;
	}

	void InitDefaults()
	{
		ThrustType = EVehicleThrustType::Fixed;
		BoneName = NAME_None;
		Offset = FVector::ZeroVector;
		ThrustAxis = FVector(1, 0, 0);
		//ThrustCurve.GetRichCurve()->AddKey(0.f, 1.f);
		//ThrustCurve.GetRichCurve()->AddKey(1.f, 1.f);
		MaxThrustForce = 1000.0f;
		MaxControlAngle = 0.f;
	}

private:
	void FillThrusterSetup(const UChaosVehicleMovementComponent& MovementComponent);

	Chaos::FSimpleThrustConfig PThrusterConfig;

};

class CHAOSVEHICLES_API UChaosVehicleSimulation
{
public:
	virtual ~UChaosVehicleSimulation()
	{
		PVehicle.Reset(nullptr);
	}

	virtual void Init(TUniquePtr<Chaos::FSimpleWheeledVehicle>& PVehicleIn)
	{
		PVehicle = MoveTemp(PVehicleIn);
	}

	virtual void UpdateConstraintHandles(TArray<FPhysicsConstraintHandle>& ConstraintHandlesIn) {}

	virtual void TickVehicle(UWorld* WorldIn, float DeltaTime, const FChaosVehicleAsyncInput& InputData, FChaosVehicleAsyncOutput& OutputData, Chaos::FRigidBodyHandle_Internal* Handle);

	/** Update the async inputs and the history ones*/
	void SyncHistoryInputs(const FChaosVehicleAsyncInput& InputData, Chaos::FRigidBodyHandle_Internal* Handle);

	/** Apply the defered forces onto the vehicles */
	virtual void ApplyDeferredForces(Chaos::FRigidBodyHandle_Internal* Handle);

	/** Update the vehicle state */
	virtual void UpdateState(float DeltaTime, const FChaosVehicleAsyncInput& InputData, Chaos::FRigidBodyHandle_Internal* Handle);

	/** Advance the vehicle simulation */
	virtual void UpdateSimulation(float DeltaTime, const FChaosVehicleAsyncInput& InputData, Chaos::FRigidBodyHandle_Internal* Handle);

	/** Fill the vehicle output state */
	virtual void FillOutputState(FChaosVehicleAsyncOutput& Output);

	/** Are enough vehicle systems specified such that physics vehicle simulation is possible */
	virtual bool CanSimulate() const { return true; }

	/** Pass control Input to the vehicle systems */
	virtual void ApplyInput(const FControlInputs& ControlInputs, float DeltaTime);

	/** Apply aerodynamic forces to vehicle body */
	virtual void ApplyAerodynamics(float DeltaTime);

	/** Apply Aerofoil forces to vehicle body */
	virtual void ApplyAerofoilForces(float DeltaTime);

	/** Apply Thruster forces to vehicle body */
	virtual void ApplyThrustForces(float DeltaTime);

	/** Apply direct control over vehicle body rotation */
	virtual void ApplyTorqueControl(float DeltaTime, const FChaosVehicleAsyncInput& InputData);

	/** Add a force to this vehicle */
	void AddForce(const FVector& Force, bool bAllowSubstepping = true, bool bAccelChange = false);

	/** Add a force at a particular position (world space when bIsLocalForce = false, body space otherwise) */
	void AddForceAtPosition(const FVector& Force, const FVector& Position, bool bAllowSubstepping = true, bool bIsLocalForce = false);

	/** Add an impulse to this vehicle */
	void AddImpulse(const FVector& Impulse, bool bVelChange);

	/** Add an impulse to this vehicle and a particular world position */
	void AddImpulseAtPosition(const FVector& Impulse, const FVector& Position);

	/** Add a torque to this vehicle */
	void AddTorqueInRadians(const FVector& Torque, bool bAllowSubstepping = true, bool bAccelChange = false);

	/** Reinitialize a wheel at runtime */
	void InitializeWheel(int WheelIndex, const Chaos::FSimpleWheelConfig* InWheelSetup);

	/** Reinitialize the physics suspension at runtime */
	void InitializeSuspension(int WheelIndex, const Chaos::FSimpleSuspensionConfig* InSuspensionSetup);

	/** Draw debug text for the wheels and suspension */
	virtual void DrawDebug3D();
	UWorld* World;

	// Physics Thread Representation of chassis rigid body
	Chaos::FRigidBodyHandle_Internal* RigidHandle;

	FVehicleState VehicleState;

	// #todo: this isn't very configurable
	TUniquePtr<Chaos::FSimpleWheeledVehicle> PVehicle;

	FDeferredForces DeferredForces;

	/** Current control inputs that is being used on the PT */
	FControlInputs VehicleInputs;

	FVehicleInputRateConfig ThrottleInputRate;
	FVehicleInputRateConfig SteeringInputRate;
	FVehicleInputRateConfig BrakeInputRate;
	FVehicleInputRateConfig HandbrakeInputRate;
	FVehicleInputRateConfig RollInputRate;
	FVehicleInputRateConfig PitchInputRate;
	FVehicleInputRateConfig YawInputRate;
};


/**
 * Base component to handle the vehicle simulation for an actor.
 */
UCLASS(Abstract, hidecategories = (PlanarMovement, "Components|Movement|Planar", Activation, "Components|Activation"))
class CHAOSVEHICLES_API UChaosVehicleMovementComponent : public UPawnMovementComponent
{
	friend struct FChaosVehicleAsyncInput;
	friend struct FChaosVehicleAsyncOutput;

	friend struct FNetworkVehicleInputs;
	friend struct FNetworkVehicleStates;

	friend class FChaosVehicleManager;
	friend class FChaosVehicleManagerAsyncCallback;

	GENERATED_UCLASS_BODY()

	//#todo: these 2 oddities seem out of place

	/** If true, the brake and reverse controls will behave in a more arcade fashion where holding reverse also functions as brake. For a more realistic approach turn this off*/
	UPROPERTY(EditAnywhere, Category = VehicleSetup)
	uint8 bReverseAsBrake : 1;

	/** If true, when reversing the throttle will behave like a brake while the vehicle moving in a backwards direction - requires bReverseAsBrake to be enabled for operation */
	UPROPERTY(EditAnywhere, Category = VehicleSetup, meta = (EditCondition = "bReverseAsBrake"))
	uint8 bThrottleAsBrake : 1;

public:
	/** Mass to set the vehicle chassis to. It's much easier to tweak vehicle settings when
	 * the mass doesn't change due to tweaks with the physics asset. [kg] */
	UPROPERTY(EditAnywhere, Category = VehicleSetup, meta = (ClampMin = "0.01", UIMin = "0.01"))
	float Mass;

	/**
	 * Enable to override the calculated COM position with your own fixed value - this prevents the vehicle handling changing when the asset changes
	 */
	UPROPERTY(EditAnywhere, Category = VehicleSetup)
	bool bEnableCenterOfMassOverride;

	/**
	 * The center of mass override value, this value overrides the calculated COM and the COM offset value in the mesh is also ignored.
	 */
	UPROPERTY(EditAnywhere, Category = VehicleSetup, meta = (EditCondition = "bEnableCenterOfMassOverride"))
	FVector CenterOfMassOverride;

	/** Chassis width used for drag force computation (cm)*/
	UPROPERTY(EditAnywhere, Category = VehicleSetup, meta = (ClampMin = "0.01", UIMin = "0.01"))
	float ChassisWidth;

	/** Chassis height used for drag force computation (cm)*/
	UPROPERTY(EditAnywhere, Category = VehicleSetup, meta = (ClampMin = "0.01", UIMin = "0.01"))
	float ChassisHeight;

	/** DragCoefficient of the vehicle chassis - force resisting forward motion at speed */
	UPROPERTY(EditAnywhere, Category = VehicleSetup)
	float DragCoefficient;

	/** DownforceCoefficient of the vehicle chassis - force pressing vehicle into ground at speed */
	UPROPERTY(EditAnywhere, Category = VehicleSetup)
	float DownforceCoefficient;

	// Drag area in Meters^2
	UPROPERTY(transient)
	float DragArea;

	// Debug drag magnitude last applied
	UPROPERTY(transient)
	float DebugDragMagnitude;

	/** Scales the vehicle's inertia in each direction (forward, right, up) */
	UPROPERTY(EditAnywhere, Category = VehicleSetup, AdvancedDisplay)
	FVector InertiaTensorScale;

	/** Option to apply some aggressive sleep logic, larger number is more agressive, 0 disables */
	UPROPERTY(EditAnywhere, Category = VehicleSetup)
	float SleepThreshold;

	/** Option to apply some aggressive sleep logic if slopes up Z is less than this value, i.e value = Cos(SlopeAngle) so 0.866 will sleep up to 30 degree slopes */
	UPROPERTY(EditAnywhere, Category = VehicleSetup, meta = (ClampMin = "0.01", UIMin = "0.01", ClampMax = "1.0", UIMax = "1.0"))
	float SleepSlopeLimit;

	/** Optional aerofoil setup - can be used for car spoilers or aircraft wings/elevator/rudder */
	UPROPERTY(EditAnywhere, Category = AerofoilSetup)
	TArray<FVehicleAerofoilConfig> Aerofoils;

	/** Optional thruster setup, use one or more as your main engine or as supplementary booster */
	UPROPERTY(EditAnywhere, Category = ThrusterSetup)
	TArray<FVehicleThrustConfig> Thrusters;

	/** Arcade style direct control of vehicle rotation via torque force */
	UPROPERTY(EditAnywhere, Category = ArcadeControl)
	FVehicleTorqueControlConfig TorqueControl;

	/** Arcade style direct control of vehicle rotation via torque force */
	UPROPERTY(EditAnywhere, Category = ArcadeControl)
	FVehicleTargetRotationControlConfig TargetRotationControl;

	/** Arcade style control of vehicle */
	UPROPERTY(EditAnywhere, Category = ArcadeControl)
	FVehicleStabilizeControlConfig StabilizeControl;

	// Used to recreate the physics if the blueprint changes.
	uint32 VehicleSetupTag;

protected:
	// True if the player is holding the handbrake
	UPROPERTY(Transient)
	uint8 bRawHandbrakeInput : 1;

	// True if the player is holding gear up
	UPROPERTY(Transient)
	uint8 bRawGearUpInput : 1;

	// True if the player is holding gear down
	UPROPERTY(Transient)
	uint8 bRawGearDownInput : 1;

	/** Was avoidance updated in this frame? */
	UPROPERTY(Transient)
	uint32 bWasAvoidanceUpdated : 1;

	UPROPERTY(Transient)
	uint8 bParkEnabled : 1;

	Chaos::ETransmissionType TransmissionType;

	UPROPERTY()
	TObjectPtr<UNetworkPhysicsComponent> NetworkPhysicsComponent = nullptr;

public:

	/** UObject interface */
	virtual void Serialize(FArchive& Ar) override;
	/** End UObject interface*/

#if WITH_EDITOR
	/** Respond to a property change in editor */
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif //WITH_EDITOR

	/** Overridden to allow registration with components NOT owned by a Pawn. */
	virtual void SetUpdatedComponent(USceneComponent* NewUpdatedComponent) override;

	/** Allow the player controller of a different pawn to control this vehicle */
	virtual void SetOverrideController(AController* OverrideController);


	/** Return true if it's suitable to create a physics representation of the vehicle at this time */
	virtual bool ShouldCreatePhysicsState() const override;

	/** Returns true if the physics state exists */
	virtual bool HasValidPhysicsState() const override;

	/** Return true if we are ready to create a vehicle, false if the setup has missing references */
	virtual bool CanCreateVehicle() const;

	/** Used to create any physics engine information for this component */
	virtual void OnCreatePhysicsState() override;

	/** Used to shut down and physics engine structure for this component */
	virtual void OnDestroyPhysicsState() override;

	/** Updates the vehicle tuning and other state such as user input. */
	virtual void PreTickGT(float DeltaTime);

	/** Stops movement immediately (zeroes velocity, usually zeros acceleration for components with acceleration). */
	virtual void StopMovementImmediately() override;

	/** Set the user input for the vehicle throttle [range 0 to 1] */
	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosVehicleMovement")
	void SetThrottleInput(float Throttle);

	/** Increase the vehicle throttle position [throttle range normalized 0 to 1] */
	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosVehicleMovement")
	void IncreaseThrottleInput(float ThrottleDelta);

	/** Decrease the vehicle throttle position  [throttle range normalized 0 to 1] */
	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosVehicleMovement")
	void DecreaseThrottleInput(float ThrottleDelta);

	/** Set the user input for the vehicle Brake [range 0 to 1] */
	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosVehicleMovement")
	void SetBrakeInput(float Brake);

	/** Set the user input for the vehicle steering [range -1 to 1] */
	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosVehicleMovement")
	void SetSteeringInput(float Steering);

	/** Set the user input for the vehicle pitch [range -1 to 1] */
	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosVehicleMovement")
	void SetPitchInput(float Pitch);

	/** Set the user input for the vehicle roll [range -1 to 1] */
	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosVehicleMovement")
	void SetRollInput(float Roll);

	/** Set the user input for the vehicle yaw [range -1 to 1] */
	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosVehicleMovement")
	void SetYawInput(float Yaw);

	/** Set the user input for handbrake */
	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosVehicleMovement")
	void SetHandbrakeInput(bool bNewHandbrake);

	/** Set the vehicle in park mode */
	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosVehicleMovement")
	void SetParked(bool bParked);

	/** Set the vehicle sleeping (bEnableSleep=true) or wake it up (bEnableSleep=false) */
	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosVehicleMovement")
	void SetSleeping(bool bEnableSleep);

	/** Set the user input for gear up */
	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosVehicleMovement")
	void SetChangeUpInput(bool bNewGearUp);

	/** Set the user input for gear down */
	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosVehicleMovement")
	void SetChangeDownInput(bool bNewGearDown);

	/** Set the user input for gear (-1 reverse, 0 neutral, 1+ forward)*/
	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosVehicleMovement")
	void SetTargetGear(int32 GearNum, bool bImmediate);

	/** Set the flag that will be used to select auto-gears */
	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosVehicleMovement")
	void SetUseAutomaticGears(bool bUseAuto);

	/** Set the flag that determines whether a controller is required to set control inputs */
	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosVehicleMovement")
	void SetRequiresControllerForInputs(bool bRequiresController);

	/** Get current gear */
	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosVehicleMovement")
	int32 GetCurrentGear() const;

	/** Get target gear */
	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosVehicleMovement")
	int32 GetTargetGear() const;

	/** Are gears being changed automatically? */
	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosVehicleMovement")
	bool GetUseAutoGears() const;

	/** How fast the vehicle is moving forward */
	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosVehicleMovement")
	float GetForwardSpeed() const;

	/** How fast the vehicle is moving forward */
	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosVehicleMovement")
	float GetForwardSpeedMPH() const;

	/** Get the user input for the vehicle throttle - can use this to feed control to a connected trailer */
	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosVehicleMovement")
	float GetThrottleInput() { return RawThrottleInput; }

	/** Get the user input for the vehicle brake - can use this to feed control to a connected trailer */
	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosVehicleMovement")
	float GetBrakeInput() { return RawBrakeInput; }

	/** Get the user input for the vehicle handbrake - can use this to feed control to a connected trailer */
	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosVehicleMovement")
	bool GetHandbrakeInput() const { return bRawHandbrakeInput; }

	/** Get the user input for the vehicle steering - can use this to feed control to a connected trailer */
	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosVehicleMovement")
	float GetSteeringInput() { return RawSteeringInput; }


	/** Is the vehicle in park mode */
	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosVehicleMovement")
	bool IsParked() const;

	/** Reset some vehicle state - call this if you are say creating pool of vehicles that get reused and you don't want to carry over the previous state */
	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosVehicleMovement")
	void ResetVehicle() { ResetVehicleState(); }

	/** Grab a snapshot of the vehicle instance dynamic state */
	void GetBaseSnapshot(FBaseSnapshotData& SnapshotOut) const;

	/** Set snapshot of vehicle instance dynamic state */
	void SetBaseSnapshot(const FBaseSnapshotData& SnapshotIn);

	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosVehicleMovement")
	void EnableSelfRighting(bool InState)
	{
		TargetRotationControl.Enabled = InState;
		TorqueControl.Enabled = InState;
		StabilizeControl.Enabled = InState;
	}

	/** location local coordinates of named bone in skeleton, apply additional offset or just use offset if no bone located */
	FVector LocateBoneOffset(const FName InBoneName, const FVector& InExtraOffset) const;

	TUniquePtr<FPhysicsVehicleOutput>& PhysicsVehicleOutput()
	{
		return PVehicleOutput;
	}

	virtual float GetSuspensionOffset(int WheelIndex) { return 0.f; }

	//----ASYNC----
	TUniquePtr<FChaosVehicleAsyncInput> SetCurrentAsyncInputOutput(int32 InputIdx, FChaosVehicleManagerAsyncOutput* CurOutput, FChaosVehicleManagerAsyncOutput* NextOutput, float Alpha, int32 VehicleManagerTimestamp);

	void SetCurrentAsyncInputOutputInternal(FChaosVehicleAsyncInput* CurInput, int32 InputIdx, FChaosVehicleManagerAsyncOutput* CurOutput, int32 VehicleManagerTimestamp);
	void SetCurrentAsyncInputOutputInternal(FChaosVehicleAsyncInput* CurInput, int32 InputIdx, FChaosVehicleManagerAsyncOutput* CurOutput, FChaosVehicleManagerAsyncOutput* NextOutput, float Alpha, int32 VehicleManagerTimestamp);

	virtual void Update(float DeltaTime);

	virtual void ResetVehicleState();

	// Get output data from Physics Thread
	virtual void ParallelUpdate(float DeltaSeconds);

	void FinalizeSimCallbackData(FChaosVehicleManagerAsyncInput& Input);

	EChaosAsyncVehicleDataType CurAsyncType;
	FChaosVehicleAsyncInput* CurAsyncInput;
	struct FChaosVehicleAsyncOutput* CurAsyncOutput;
	struct FChaosVehicleAsyncOutput* NextAsyncOutput;
	float OutputInterpAlpha;

	struct FAsyncOutputWrapper
	{
		int32 Idx;
		int32 Timestamp;

		FAsyncOutputWrapper()
			: Idx(INDEX_NONE)
			, Timestamp(INDEX_NONE)
		{
		}
	};
	TArray<FAsyncOutputWrapper> OutputsWaitingOn;

protected:

	// replicated state of vehicle 
	UPROPERTY(Transient, Replicated)
	FVehicleReplicatedState ReplicatedState;

	// accumulator for RB replication errors 
	float AngErrorAccumulator;

	// What the player has the steering set to. Range -1...1
	UPROPERTY(Transient)
	float RawSteeringInput;

	// What the player has the accelerator set to. Range -1...1
	UPROPERTY(Transient)
	float RawThrottleInput;

	// What the player has the brake set to. Range -1...1
	UPROPERTY(Transient)
	float RawBrakeInput;

	// What the player has the pitch set to. Range -1...1
	UPROPERTY(Transient)
	float RawPitchInput;

	// What the player has the roll set to. Range -1...1
	UPROPERTY(Transient)
	float RawRollInput;

	// What the player has the yaw set to. Range -1...1
	UPROPERTY(Transient)
	float RawYawInput;

	// Steering output to physics system. Range -1...1
	UPROPERTY(Transient)
	float SteeringInput;

	// Accelerator output to physics system. Range 0...1
	UPROPERTY(Transient)
	float ThrottleInput;

	// Brake output to physics system. Range 0...1
	UPROPERTY(Transient)
	float BrakeInput;

	// Body Pitch output to physics system. Range -1...1
	UPROPERTY(Transient)
	float PitchInput;

	// Body Roll output to physics system. Range -1...1
	UPROPERTY(Transient)
	float RollInput;

	// Body Yaw output to physics system. Range -1...1
	UPROPERTY(Transient)
	float YawInput;

	// Handbrake output to physics system. Range 0...1
	UPROPERTY(Transient)
	float HandbrakeInput;

	// Bypass the need for a controller in order for the controls to be processed.
	UPROPERTY(EditAnywhere, Category = VehicleInput)
	bool bRequiresControllerForInputs;

	// How much to press the brake when the player has release throttle
	UPROPERTY(EditAnywhere, Category = VehicleInput)
	float IdleBrakeInput;

	// Auto-brake when absolute vehicle forward speed is less than this (cm/s)
	UPROPERTY(EditAnywhere, Category = VehicleInput)
	float StopThreshold;

	// Auto-brake when vehicle forward speed is opposite of player input by at least this much (cm/s)
	UPROPERTY(EditAnywhere, Category = VehicleInput)
	float WrongDirectionThreshold;

public:
	// Rate at which input throttle can rise and fall
	UPROPERTY(EditAnywhere, Category = VehicleInput, AdvancedDisplay)
	FVehicleInputRateConfig ThrottleInputRate;

	// Rate at which input brake can rise and fall
	UPROPERTY(EditAnywhere, Category = VehicleInput, AdvancedDisplay)
	FVehicleInputRateConfig BrakeInputRate;

	// Rate at which input steering can rise and fall
	UPROPERTY(EditAnywhere, Category = VehicleInput, AdvancedDisplay)
	FVehicleInputRateConfig SteeringInputRate;

	// Rate at which input handbrake can rise and fall
	UPROPERTY(EditAnywhere, Category = VehicleInput, AdvancedDisplay)
	FVehicleInputRateConfig HandbrakeInputRate;

	// Rate at which input pitch can rise and fall
	UPROPERTY(EditAnywhere, Category = VehicleInput, AdvancedDisplay)
	FVehicleInputRateConfig PitchInputRate;

	// Rate at which input roll can rise and fall
	UPROPERTY(EditAnywhere, Category = VehicleInput, AdvancedDisplay)
	FVehicleInputRateConfig RollInputRate;

	// Rate at which input yaw can rise and fall
	UPROPERTY(EditAnywhere, Category = VehicleInput, AdvancedDisplay)
	FVehicleInputRateConfig YawInputRate;

#if UE_WITH_REMOTE_OBJECT_HANDLE
	void ApplyMigratedVehicleState();
#endif
protected:

	// input related

	/** Compute steering input */
	float CalcSteeringInput();

	/** Compute throttle & brake input */
	void CalcThrottleBrakeInput(float& ThrottleOut, float& BrakeOut);

	/** Compute handbrake input */
	float CalcHandbrakeInput();

	/** Compute pitch input */
	float CalcPitchInput();

	/** Compute roll input */
	float CalcRollInput();

	/** Compute yaw input */
	float CalcYawInput();

	/** Compute throttle inputs */
	float CalcThrottleUpInput();
	float CalcThrottleDownInput();

	/**
	* Clear all interpolated inputs to default values.
	* Raw input won't be cleared, the vehicle may resume input based movement next frame.
	*/
	virtual void ClearInput();

	/**
	* Clear all raw inputs to default values.
	* Interpolated input won't be cleared, the vehicle will begin interpolating to no input.
	*/
	virtual void ClearRawInput();

	void ClearAllInput()
	{
		ClearRawInput();
		ClearInput();
	}

	// Update

	/** Read current state for simulation */
	virtual void UpdateState(float DeltaTime);

	/** Option to aggressively sleep the vehicle */
	virtual void ProcessSleeping(const FControlInputs& ControlInputs);

	/** Pass current state to server */
	UFUNCTION(reliable, server, WithValidation)
	void ServerUpdateState(float InSteeringInput, float InThrottleInput, float InBrakeInput
			, float InHandbrakeInput, int32 InCurrentGear, float InRollInput, float InPitchInput, float InYawInput);

	// Setup

	/** Get our controller */
	virtual AController* GetController() const override;

	/** Retrieve the player controller of the vehicle component */
	APlayerController* GetPlayerController() const;

	/** Get the mesh this vehicle is tied to */
	class UMeshComponent* GetMesh() const;

	/** Get Mesh cast as USkeletalMeshComponent, may return null if cast fails */
	USkeletalMeshComponent* GetSkeletalMesh();

	/** Get Mesh cast as UStaticMeshComponent, may return null if cast fails */
	UStaticMeshComponent* GetStaticMesh();

	/** Create and setup the Chaos vehicle */
	virtual void CreateVehicle();

	virtual TUniquePtr<Chaos::FSimpleWheeledVehicle> CreatePhysicsVehicle();

	/** Skeletal mesh needs some special handling in the vehicle case */
	virtual void FixupSkeletalMesh() {}

	/** Allocate and setup the Chaos vehicle */
	virtual void SetupVehicle(TUniquePtr<Chaos::FSimpleWheeledVehicle>& PVehicle);

	/** Do some final setup after the Chaos vehicle gets created */
	virtual void PostSetupVehicle();

	/** Adjust the Chaos Physics mass */
	virtual void SetupVehicleMass();

	void UpdateMassProperties(FBodyInstance* BI);

	/** When vehicle is created we want to compute some helper data like drag area, etc.... Derived classes should use this to properly compute things like engine RPM */
	virtual void ComputeConstants();

	// Debug

	void ShowDebugInfo(class AHUD* HUD, class UCanvas* Canvas, const class FDebugDisplayInfo& DisplayInfo, float& YL, float& YPos);

	/** Draw debug text for the wheels and suspension */
	virtual void DrawDebug(UCanvas* Canvas, float& YL, float& YPos);

	// draw 2D debug line to UI canvas
	void DrawLine2D(UCanvas* Canvas, const FVector2D& StartPos, const FVector2D& EndPos, FColor Color, float Thickness = 1.f);

#if UE_WITH_REMOTE_OBJECT_HANDLE
	ENGINE_API bool CanSendMigratedPhysicsState(const FUObjectMigrationContext* MigrationContext, EObjectMigrationSendType ObjMigrationSendType) const;

	// migrates physics related state (such as vehicle ouput)
	ENGINE_API virtual void MigratePhysicsState(FArchive& Ar);

	ENGINE_API virtual void PostMigrate(const struct FUObjectMigrationContext& MigrationContext) override;
#endif

	float GetForwardAcceleration()
	{
		return VehicleState.ForwardsAcceleration;
	}

	FBodyInstance* GetBodyInstance();
	const FBodyInstance* GetBodyInstance() const;
	FVehicleState VehicleState;							/* Useful vehicle state captured at start of frame */
	TUniquePtr<FPhysicsVehicleOutput> PVehicleOutput;	/* physics simulation data output from the async physics thread */

	/** Handle for delegate registered on mesh component */
	FDelegateHandle MeshOnPhysicsStateChangeHandle;

protected:

	TUniquePtr<UChaosVehicleSimulation> VehicleSimulationPT;	/* simulation code running on the physics thread async callback */

	UPROPERTY(transient, Replicated)
	TObjectPtr<AController> OverrideController;

	const Chaos::FSimpleAerodynamicsConfig& GetAerodynamicsConfig()
	{
		FillAerodynamicsSetup();
		return PAerodynamicsSetup;
	}

	void FillAerodynamicsSetup()
	{
		PAerodynamicsSetup.DragCoefficient = this->DragCoefficient;
		PAerodynamicsSetup.DownforceCoefficient = this->DownforceCoefficient;
		PAerodynamicsSetup.AreaMetresSquared = Chaos::Cm2ToM2(this->DragArea);
	}

	void WakeAllEnabledRigidBodies();
	void PutAllEnabledRigidBodiesToSleep();


	Chaos::FSimpleAerodynamicsConfig PAerodynamicsSetup;
	int32 TargetGear;

	float PrevSteeringInput;
	float PrevReplicatedSteeringInput;
	bool bUsingNetworkPhysicsPrediction;

private:
#if UE_WITH_REMOTE_OBJECT_HANDLE
	bool bIsMigrated = false;
	FNetworkVehicleStates MigratedVehicleStates;
	bool bIsVehicleStatesMigrated = false;
#endif
};
