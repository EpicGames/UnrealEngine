// Copyright Epic Games, Inc. All Rights Reserved.

/*
 * Component to handle the vehicle simulation for an actor
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/ScriptMacros.h"
#include "EngineDefines.h"
#include "SimpleVehicle.h"
#include "VehicleUtility.h"
#include "Curves/CurveFloat.h"

#include "ChaosVehicleWheel.generated.h"

class UPhysicalMaterial;
class FChaosVehicleManager;
class UChaosWheeledVehicleMovementComponent;

	UENUM()
	enum class ESweepShape : uint8
	{
		/** Use ray to determine suspension length to ground - fastest */
		Raycast		UMETA(DisplayName = "Raycast"),

		/** Use sphere to determine suspension length to ground */
		Spherecast	UMETA(DisplayName = "Spherecast"),

		/** Use wheel collision shape to determine suspension length to ground - Slowest */
		Shapecast	UMETA(DisplayName = "Shapecast")
	};

	UENUM()
	enum class ESweepType : uint8
	{
		/** Sweeps against simple geometry only */
		SimpleSweep				UMETA(DisplayName = "SimpleSweep"),
		/** Sweeps against complex geometry only */
		ComplexSweep			UMETA(DisplayName = "ComplexSweep")
	};

	UENUM()
	enum class EAxleType : uint8
	{
		Undefined = 0,
		Front,
		Rear
	};

	UENUM()
	enum class ETorqueCombineMethod : uint8
	{
		/** External torque value has no effect - default **/
		None = 0,
		/** completely replace existing torques, can set and forget will apply same torque every frame until zeroed */
		Override,
		/** Combine external torque with existing torques, must set external torque every frame */
		Additive
	};

	UCLASS(BlueprintType, Blueprintable)
		class CHAOSVEHICLES_API UChaosVehicleWheel : public UObject
	{
		GENERATED_UCLASS_BODY()

		/**
			* Static mesh with collision setup for wheel, will be used to create wheel shape
			* (if empty, sphere will be added as wheel shape, check bDontCreateShape flag)
			*/
		UPROPERTY(EditDefaultsOnly, Category = Shape)
		TObjectPtr<class UStaticMesh> CollisionMesh;

		/** If left undefined then the bAffectedByEngine value is used, if defined then bAffectedByEngine is ignored and the differential setup on the vehicle defines which wheels get power from the engine */
		UPROPERTY(EditAnywhere, Category = Wheel)
		EAxleType AxleType;

		/**
		 * If BoneName is specified, offset the wheel from the bone's location.
		 * Otherwise this offsets the wheel from the vehicle's origin.
		 */
		UPROPERTY(EditAnywhere, Category = Wheel)
		FVector Offset;

		/** Radius of the wheel */
		UPROPERTY(EditAnywhere, Category = Wheel, meta = (ClampMin = "0.01", UIMin = "0.01"))
		float WheelRadius;

		/** Width of the wheel */
		UPROPERTY(EditAnywhere, Category = Wheel, meta = (ClampMin = "0.01", UIMin = "0.01"))
		float WheelWidth;

		/** Mass of the wheel Kg */
		UPROPERTY(EditAnywhere, Category = Wheel, meta = (ClampMin = "0.01", UIMin = "0.01"))
		float WheelMass;

		/** Tyre Cornering Ability */
		UPROPERTY(EditAnywhere, Category = Wheel)
		float CorneringStiffness;

		/** Friction Force Multiplier */
		UPROPERTY(EditAnywhere, Category = Wheel)
		float FrictionForceMultiplier;

		/** Wheel Lateral Skid Grip Loss, lower number less grip on skid */
		UPROPERTY(EditAnywhere, Category = Wheel, meta = (ClampMin = "0.0", UIMin = "0.0", ClampMax = "1.0", UIMax = "1.0"))
		float SideSlipModifier;

		/** Wheel Longitudinal Slip Threshold */
		UPROPERTY(EditAnywhere, Category = Wheel, meta = (ClampMin = "0.0", UIMin = "0.0"))
		float SlipThreshold;

		/** Wheel Lateral Skid Threshold */
		UPROPERTY(EditAnywhere, Category = Wheel, meta = (ClampMin = "0.0", UIMin = "0.0"))
		float SkidThreshold;

		// steer angle in degrees for this wheel
		UPROPERTY(EditAnywhere, Category = WheelsSetup)
		float MaxSteerAngle;

		/** Whether steering should affect this wheel */
		UPROPERTY(EditAnywhere, Category = WheelsSetup)
		bool bAffectedBySteering;

		/** Whether brake should affect this wheel */
		UPROPERTY(EditAnywhere, Category = Wheel)
		bool bAffectedByBrake;

		/** Whether handbrake should affect this wheel */
		UPROPERTY(EditAnywhere, Category = Wheel)
		bool bAffectedByHandbrake;

		/** Whether engine should power this wheel */
		UPROPERTY(EditAnywhere, Category = Wheel)
		bool bAffectedByEngine;

		/** Advanced Braking System Enabled */
		UPROPERTY(EditAnywhere, Category = Wheel)
		bool bABSEnabled;

		/** Straight Line Traction Control Enabled */
		UPROPERTY(EditAnywhere, Category = Wheel)
		bool bTractionControlEnabled;

		/** Max Wheelspin rotation rad/sec */
		UPROPERTY(EditAnywhere, Category = Wheel)
		float MaxWheelspinRotation;

		/** Determines how the SetDriveTorque/SetBrakeTorque inputs are combined with the internal torques */
		UPROPERTY(EditAnywhere, Category = Wheel)
		ETorqueCombineMethod ExternalTorqueCombineMethod;

		UPROPERTY(EditAnywhere, Category = Setup)
		FRuntimeFloatCurve LateralSlipGraph;

		/** Local body direction in which where suspension forces are applied (typically along -Z-axis) */
		UPROPERTY(EditAnywhere, Category = Suspension)
		FVector SuspensionAxis;

		/** Vertical offset from where suspension forces are applied (along Z-axis) */
		UPROPERTY(EditAnywhere, Category = Suspension)
		FVector SuspensionForceOffset;

		/** How far the wheel can go above the resting position */
		UPROPERTY(EditAnywhere, Category = Suspension)
		float SuspensionMaxRaise;

		/** How far the wheel can drop below the resting position */
		UPROPERTY(EditAnywhere, Category = Suspension)
		float SuspensionMaxDrop;

		/** Suspension damping, larger value causes the suspension to come to rest faster [range 0 to 1] */
		UPROPERTY(EditAnywhere, Category = Suspension)
		float SuspensionDampingRatio;

		/**
		 *	Amount wheel load effects wheel friction. 
			At 0 wheel friction is completely independent of the loading on the wheel (This is artificial as it always assumes even balance between all wheels)
			At 1 wheel friction is based on the force pressing wheel into the ground. This is more realistic.
			Lower value cures lift off over-steer, generally makes vehicle easier to handle under extreme motions.
		 */
		UPROPERTY(EditAnywhere, Category = Suspension, meta = (ClampMin = "0.0", UIMin = "0.0", ClampMax = "1.0", UIMax = "1.0"))
		float WheelLoadRatio;

		/** Spring Force (N/m) */
		UPROPERTY(EditAnywhere, Category = Suspension)
		float SpringRate;

		/** Spring Preload (N/m) */
		UPROPERTY(EditAnywhere, Category = Suspension)
		float SpringPreload;

		/** Smooth suspension [0-off, 10-max] - Warning might cause momentary visual inter-penetration of the wheel against objects/terrain */
		UPROPERTY(EditAnywhere, Category = Suspension, meta = (ClampMin = "0.0", UIMin = "0", ClampMax = "10.0", UIMax = "10"))
		int SuspensionSmoothing;

		/** Anti-roll effect */
		UPROPERTY(EditAnywhere, Category = Suspension, meta = (ClampMin = "0.0", UIMin = "0", ClampMax = "1.0", UIMax = "1"))
		float RollbarScaling;

		/** Wheel suspension trace type, defaults to ray trace */
		UPROPERTY(EditAnywhere, Category = Suspension)
		ESweepShape SweepShape;

		/** Whether wheel suspension considers simple, complex */
		UPROPERTY(EditAnywhere, Category = Suspension)
		ESweepType SweepType;

		/** max brake torque for this wheel (Nm) */
		UPROPERTY(EditAnywhere, Category = Brakes)
		float MaxBrakeTorque;

		/**
		 *	Max handbrake brake torque for this wheel (Nm). A handbrake should have a stronger brake torque
		 *	than the brake. This will be ignored for wheels that are not affected by the handbrake.
		 */
		UPROPERTY(EditAnywhere, Category = Brakes)
		float MaxHandBrakeTorque;

		/** The vehicle that owns us */
		UPROPERTY(transient)
		TObjectPtr<class UChaosWheeledVehicleMovementComponent> VehicleComponent;

		// Our index in the vehicle's (and setup's) wheels array
		UPROPERTY(transient)
		int32 WheelIndex;

		// Longitudinal slip experienced by the wheel
		UPROPERTY(transient)
		float DebugLongSlip;

		// Lateral slip experienced by the wheel
		UPROPERTY(transient)
		float DebugLatSlip;

		// How much force the tire experiences at rest divided by how much force it is experiencing now
		UPROPERTY(transient)
		float DebugNormalizedTireLoad;

		//How much force the tire is experiencing now
		float DebugTireLoad;

		// Wheel torque
		UPROPERTY(transient)
		float DebugWheelTorque;

		// Longitudinal force the wheel is applying to the chassis
		UPROPERTY(transient)
		float DebugLongForce;

		// Lateral force the wheel is applying to the chassis
		UPROPERTY(transient)
		float DebugLatForce;

		// Worldspace location of this wheel
		UPROPERTY(transient)
		FVector Location;

		// Worldspace location of this wheel last frame
		UPROPERTY(transient)
		FVector OldLocation;

		// Current velocity of the wheel center (change in location over time)
		UPROPERTY(transient)
		FVector Velocity;

		UFUNCTION(BlueprintCallable, Category = "Game|Components|WheeledVehicleMovement")
		float GetSteerAngle() const;

		UFUNCTION(BlueprintCallable, Category = "Game|Components|WheeledVehicleMovement")
		float GetRotationAngle() const;

		UFUNCTION(BlueprintCallable, Category = "Game|Components|WheeledVehicleMovement")
		float GetRotationAngularVelocity() const;

		UFUNCTION(BlueprintCallable, Category = "Game|Components|WheeledVehicleMovement")
		float GetSuspensionOffset() const;

		UFUNCTION(BlueprintCallable, Category = "Game|Components|WheeledVehicleMovement")
		float GetWheelRadius() const;

		UFUNCTION(BlueprintCallable, Category = "Game|Components|WheeledVehicleMovement")
		float GetWheelAngularVelocity() const;

		UFUNCTION(BlueprintCallable, Category = "Game|Components|WheeledVehicleMovement")
		FVector GetSuspensionAxis() const;

		UFUNCTION(BlueprintCallable, Category = "Game|Components|WheeledVehicleMovement")
		bool IsInAir() const;

		UFUNCTION(BlueprintCallable, Category = "Game|Components|WheeledVehicleMovement")
		EAxleType GetAxleType() { return AxleType; }


		/**
		 * Initialize this wheel instance
		 */
		virtual void Init(class UChaosWheeledVehicleMovementComponent* InVehicleSim, int32 InWheelIndex);

		/**
		 * Notify this wheel it will be removed from the scene
		 */
		virtual void Shutdown();

		/**
		 * Get the Axle setup we were created from
		 */
		struct FChaosWheelSetup& GetWheelSetup();

		/**
		 * Tick this class when the vehicle ticks
		 */
		virtual void Tick(float DeltaTime);

#if WITH_EDITOR

		/**
		 * Respond to a property change in editor
		 */
		virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;

#endif //WITH_EDITOR

	protected:
	
		/**
		 * Get the wheel's location in physics land
		 */
		FVector GetPhysicsLocation();

	private:

		FChaosVehicleManager* GetVehicleManager() const;

		void FillWheelSetup()
		{
			// Perform any unit conversions here; between editable property and simulation system
			PWheelConfig.Offset = this->Offset;
			//PWheelConfig.WheelMass = this->WheelMass;
			PWheelConfig.WheelRadius = this->WheelRadius;
			PWheelConfig.WheelWidth = this->WheelWidth;
			PWheelConfig.WheelMass = this->WheelMass;
			PWheelConfig.MaxSteeringAngle = this->MaxSteerAngle;
			PWheelConfig.MaxBrakeTorque = this->MaxBrakeTorque;
			PWheelConfig.HandbrakeTorque = this->MaxHandBrakeTorque;
			PWheelConfig.SteeringEnabled = this->bAffectedBySteering;
			PWheelConfig.BrakeEnabled = this->bAffectedByBrake;
			PWheelConfig.HandbrakeEnabled = this->bAffectedByHandbrake;
			PWheelConfig.EngineEnabled = this->bAffectedByEngine;	// may be modified later by vehicle differential override setup
			PWheelConfig.TorqueRatio = 0.f;							// calculated later after all wheel info is known
			PWheelConfig.ABSEnabled = this->bABSEnabled;
			PWheelConfig.TractionControlEnabled = this->bTractionControlEnabled;
			PWheelConfig.MaxSpinRotation = this->MaxWheelspinRotation;
			PWheelConfig.AxleType = static_cast<Chaos::FSimpleWheelConfig::EAxleType>(this->AxleType);
			PWheelConfig.FrictionMultiplier = this->FrictionForceMultiplier;
			PWheelConfig.CorneringStiffness = this->CorneringStiffness * 10000.0f;
			PWheelConfig.SideSlipModifier = this->SideSlipModifier;
			PWheelConfig.SlipThreshold = this->SlipThreshold;
			PWheelConfig.SkidThreshold = this->SkidThreshold;
			PWheelConfig.ExternalTorqueCombineMethod = static_cast<Chaos::FSimpleWheelConfig::EExternalTorqueCombineMethod>(this->ExternalTorqueCombineMethod);

			PWheelConfig.LateralSlipGraph.Empty();
			float NumSamples = 20;
			float MinTime = 0.f, MaxTime = 0.f;
			this->LateralSlipGraph.GetRichCurveConst()->GetTimeRange(MinTime, MaxTime);
			if (MaxTime > 0.0f)
			{
				for (float X = 0; X <= MaxTime; X += (MaxTime / NumSamples))
				{
					float MinVal = 0.f, MaxVal = 0.f;
					this->LateralSlipGraph.GetRichCurveConst()->GetValueRange(MinVal, MaxVal);
					float Y = this->LateralSlipGraph.GetRichCurveConst()->Eval(X) * 10000.0f;
					PWheelConfig.LateralSlipGraph.Add(Chaos::FVec2(X, Y));
				}
			}

		}

		void FillSuspensionSetup()
		{
			PSuspensionConfig.SuspensionAxis = this->SuspensionAxis;

			PSuspensionConfig.SuspensionForceOffset = this->SuspensionForceOffset;
			PSuspensionConfig.SuspensionMaxRaise = this->SuspensionMaxRaise;
			PSuspensionConfig.SuspensionMaxDrop = this->SuspensionMaxDrop;
			PSuspensionConfig.SpringRate = Chaos::MToCm(this->SpringRate);
			PSuspensionConfig.SpringPreload = Chaos::MToCm(this->SpringPreload);

			PSuspensionConfig.DampingRatio = this->SuspensionDampingRatio;
			PSuspensionConfig.WheelLoadRatio = this->WheelLoadRatio;

			PSuspensionConfig.SuspensionSmoothing = this->SuspensionSmoothing;

			// These are calculated later from the PSuspensionConfig.DampingRatio
			//		PSuspensionConfig.ReboundDamping
			//		PSuspensionConfig.CompressionDamping
		}

		Chaos::FSimpleWheelConfig PWheelConfig;
		Chaos::FSimpleSuspensionConfig PSuspensionConfig;

	public:

		const Chaos::FSimpleWheelConfig& GetPhysicsWheelConfig()
		{
			FillWheelSetup();
			return PWheelConfig;
		}

		const Chaos::FSimpleSuspensionConfig& GetPhysicsSuspensionConfig()
		{
			FillSuspensionSetup();
			return PSuspensionConfig;
		}

		/** Get contact surface material */
		UPhysicalMaterial* GetContactSurfaceMaterial();

	};

