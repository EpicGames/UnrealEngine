// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Physics/PhysicsInterfaceCore.h"
#include "UObject/ObjectMacros.h"
#include "ChaosVehicleMovementComponent.h"
#include "Curves/CurveFloat.h"
#include "VehicleUtility.h"
#include "Chaos/PBDSuspensionConstraints.h"
#include "PhysicsProxy/SingleParticlePhysicsProxyFwd.h"
#include "ChaosWheeledVehicleMovementComponent.generated.h"

#if VEHICLE_DEBUGGING_ENABLED
UE_DISABLE_OPTIMIZATION
#endif

struct FWheeledVehicleDebugParams
{
	bool ShowWheelCollisionNormal = false;
	bool ShowSuspensionRaycasts = false;
	bool ShowSuspensionLimits = false;
	bool ShowWheelForces = false;
	bool ShowSuspensionForces = false;
	bool ShowBatchQueryExtents = false;
	bool ShowRaycastComponent = false;
	bool ShowRaycastMaterial = false;
	bool CacheSuspensionOffset = true;
	int TraceTypeOverride = 0;

	bool DisableSuspensionForces = false;
	bool DisableFrictionForces = false;
	bool DisableRollbarForces = false;
	bool DisableConstraintSuspension = false;

	float ThrottleOverride = 0.f;
	float SteeringOverride = 0.f;

	bool ResetPerformanceMeasurements = false;

	float OverlapTestExpansionXY = 100.f;
	float OverlapTestExpansionZ = 50.f;
};

/**
 * There is too much information for one screen full of debug data, so sub-pages of information are available 
 * Advance through pages using p.Vehicles.NextDebugPage | p.Vehicles.PrevDebugPage which can be hooked
 * up to the keyboard or a controller in blueprint using execCommand
 */
enum EDebugPages : uint8
{
	BasicPage = 0,
	PerformancePage,
	SteeringPage,
	FrictionPage,
	SuspensionPage,
	TransmissionPage,

	MaxDebugPages	// keep as last value
};

UENUM()
enum class EVehicleDifferential : uint8
{
	Undefined,
	AllWheelDrive,
	FrontWheelDrive,
	RearWheelDrive,
};



/**
 * Structure containing information about the status of a single wheel of the vehicle.
 */
USTRUCT(BlueprintType)
struct CHAOSVEHICLES_API FWheelStatus
{
	GENERATED_BODY()

	/** This wheel is in contact with the ground */
	UPROPERTY()
	bool bInContact;

	/** Wheel contact point */
	UPROPERTY()
	FVector ContactPoint;

	/** Wheel contact location */
	UPROPERTY()
	FVector HitLocation;

	/** Material that wheel is in contact with */
	UPROPERTY()
	TWeakObjectPtr<class UPhysicalMaterial> PhysMaterial;

	/** Normalized suspension length at this wheel */
	UPROPERTY()
	float NormalizedSuspensionLength;

	/** Spring Force that is occurring at wheel suspension */
	UPROPERTY()
	float SpringForce;

	/** Slip angle at the wheel - difference between wheel local direction and velocity at wheel */
	UPROPERTY()
	float SlipAngle;

	/** Is the wheel slipping */
	UPROPERTY()
	bool bIsSlipping;

	/** Magnitude of slippage of wheel, difference between wheel speed and ground speed */
	UPROPERTY()
	float SlipMagnitude;


	/** Is the wheel skidding */
	UPROPERTY()
	bool bIsSkidding;

	/** Magnitude of skid */
	UPROPERTY()
	float SkidMagnitude;

	/** Direction of skid, i.e. normalized direction */
	UPROPERTY()
	FVector SkidNormal;

	/** Drive torque currently applied at wheel */
	UPROPERTY()
	float DriveTorque;

	/** Brake torque currently applied at wheel */
	UPROPERTY()
	float BrakeTorque;

	/** Is the ABS currently engaged - useful for audio Q's */
	UPROPERTY()
	bool bABSActivated;

	FWheelStatus()
	{
		Init();
	}

	explicit FWheelStatus(EForceInit InInit)
	{
		Init();
	}

	explicit FWheelStatus(ENoInit NoInit)
	{
		bIsValid = false;
	}

	void Init()
	{
		SlipAngle = 0.0f;
		bInContact = false;
		bIsSlipping = false;
		bIsSkidding = false;
		SlipMagnitude = 0.f;
		SkidMagnitude = 0.f;
		NormalizedSuspensionLength = 1.f;
		SpringForce = 0.f;
		SkidNormal = FVector::ZeroVector;
		ContactPoint = FVector::ZeroVector;
		HitLocation = FVector::ZeroVector;
		bIsValid = false;
		bABSActivated = false;
		DriveTorque = 0.0f;
		BrakeTorque = 0.0f;
	}

	FString ToString() const;

	bool bIsValid;
};

USTRUCT()
struct CHAOSVEHICLES_API FVehicleDifferentialConfig
{
	GENERATED_USTRUCT_BODY()

	FVehicleDifferentialConfig()
	{
		InitDefaults();
	}

	/** Type of differential */
	UPROPERTY(EditAnywhere, Category=Setup)
	EVehicleDifferential DifferentialType;
	
	/** Ratio of torque split between front and rear (<0.5 means more to front, >0.5 means more to rear, works only with 4W type) */
	UPROPERTY(EditAnywhere, Category = Setup, meta = (ClampMin = "0.0", UIMin = "0.0", ClampMax = "1.0", UIMax = "1.0"))
	float FrontRearSplit;

	const Chaos::FSimpleDifferentialConfig& GetPhysicsDifferentialConfig()
	{
		FillDifferentialSetup();
		return PDifferentialConfig;
	}

	void InitDefaults()
	{
		DifferentialType = EVehicleDifferential::RearWheelDrive;
		FrontRearSplit = 0.5f;
	}

	void FillDifferentialSetup()
	{
		PDifferentialConfig.DifferentialType = static_cast<Chaos::EDifferentialType>(this->DifferentialType);
		PDifferentialConfig.FrontRearSplit = this->FrontRearSplit;
	}

	Chaos::FSimpleDifferentialConfig PDifferentialConfig;

};

USTRUCT()
struct CHAOSVEHICLES_API FVehicleEngineConfig
{
	GENERATED_USTRUCT_BODY()

	FVehicleEngineConfig()
	{
		InitDefaults();
	}

	/** Torque [Normalized 0..1] for a given RPM */
	UPROPERTY(EditAnywhere, Category = Setup)
	FRuntimeFloatCurve TorqueCurve;

	/** Max Engine Torque (Nm) is multiplied by TorqueCurve */
	UPROPERTY(EditAnywhere, Category = Setup)
	float MaxTorque;

	/** Maximum revolutions per minute of the engine */
	UPROPERTY(EditAnywhere, Category = Setup, meta = (ClampMin = "0.01", UIMin = "0.01"))
	float MaxRPM;

	/** Idle RMP of engine then in neutral/stationary */
	UPROPERTY(EditAnywhere, Category = Setup, meta = (ClampMin = "0.01", UIMin = "0.01"))
	float EngineIdleRPM;

	/** Braking effect from engine, when throttle released */
	UPROPERTY(EditAnywhere, Category = Setup)
	float EngineBrakeEffect;

	/** Affects how fast the engine RPM speed up */
	UPROPERTY(EditAnywhere, Category = Setup, meta = (ClampMin = "0.01", UIMin = "0.01"))
	float EngineRevUpMOI;

	/** Affects how fast the engine RPM slows down */
	UPROPERTY(EditAnywhere, Category = Setup, meta = (ClampMin = "0.01", UIMin = "0.01"))
	float EngineRevDownRate;

	const Chaos::FSimpleEngineConfig& GetPhysicsEngineConfig()
	{
		FillEngineSetup();
		return PEngineConfig;
	}

	void InitDefaults()
	{
		MaxTorque = 300.0f;
		MaxRPM = 4500.0f;
		EngineIdleRPM = 1200.0f;
		EngineBrakeEffect = 0.05f;
		EngineRevUpMOI = 5.0f;
		EngineRevDownRate = 600.0f;
	}

	float GetTorqueFromRPM(float EngineRPM)
	{
		// The source curve does not need to be normalized, however we are normalizing it when it is passed on,
		// since it's the MaxRPM and MaxTorque values that determine the range of RPM and Torque
		float MinVal = 0.f, MaxVal = 0.f;
		this->TorqueCurve.GetRichCurveConst()->GetValueRange(MinVal, MaxVal);
		return TorqueCurve.GetRichCurve()->Eval(EngineRPM) / MaxVal * MaxTorque;
	}
private:

	void FillEngineSetup()
	{
		// The source curve does not need to be normalized, however we are normalizing it when it is passed on,
		// since it's the MaxRPM and MaxTorque values that determine the range of RPM and Torque
		PEngineConfig.TorqueCurve.Empty();
		float NumSamples = 20;
		for (float X = 0; X <= this->MaxRPM; X+= (this->MaxRPM / NumSamples))
		{ 
			float MinVal = 0.f, MaxVal = 0.f;
			this->TorqueCurve.GetRichCurveConst()->GetValueRange(MinVal, MaxVal);
			float Y = this->TorqueCurve.GetRichCurveConst()->Eval(X) / MaxVal;
			PEngineConfig.TorqueCurve.AddNormalized(Y);
		}
		PEngineConfig.MaxTorque = this->MaxTorque;
		PEngineConfig.MaxRPM = this->MaxRPM;
		PEngineConfig.EngineIdleRPM = this->EngineIdleRPM;
		PEngineConfig.EngineBrakeEffect = this->EngineBrakeEffect;
		PEngineConfig.EngineRevUpMOI = this->EngineRevUpMOI;
		PEngineConfig.EngineRevDownRate = this->EngineRevDownRate;
	}

	Chaos::FSimpleEngineConfig PEngineConfig;

};

USTRUCT()
struct CHAOSVEHICLES_API FVehicleTransmissionConfig
{
	GENERATED_USTRUCT_BODY()

	FVehicleTransmissionConfig()
	{
		InitDefaults();
	}

	friend class UChaosVehicleWheel;

	/** Whether to use automatic transmission */
	UPROPERTY(EditAnywhere, Category = VehicleSetup, meta=(DisplayName = "Automatic Transmission"))
	bool bUseAutomaticGears;

	UPROPERTY(EditAnywhere, Category = VehicleSetup, meta = (DisplayName = "Automatic Reverse"))
	bool bUseAutoReverse;

	/** The final gear ratio multiplies the transmission gear ratios.*/
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Setup)
	float FinalRatio;

	/** Forward gear ratios */
	UPROPERTY(EditAnywhere, Category = Setup, AdvancedDisplay)
	TArray<float> ForwardGearRatios;

	/** Reverse gear ratio(s) */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Setup)
	TArray<float> ReverseGearRatios;

	/** Engine Revs at which gear up change ocurrs */
	UPROPERTY(EditAnywhere, meta = (ClampMin = "0.0", UIMin = "0.0", ClampMax = "50000.0", UIMax = "50000.0"), Category = Setup)
	float ChangeUpRPM;

	/** Engine Revs at which gear down change ocurrs */
	UPROPERTY(EditAnywhere, meta = (ClampMin = "0.0", UIMin = "0.0", ClampMax = "50000.0", UIMax = "50000.0"), Category = Setup)
	float ChangeDownRPM;

	/** Time it takes to switch gears (seconds) */
	UPROPERTY(EditAnywhere, Category = Setup, meta = (ClampMin = "0.0", UIMin = "0.0"))
	float GearChangeTime;

	/** Mechanical frictional losses mean transmission might operate at 0.94 (94% efficiency) */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Setup)
	float TransmissionEfficiency;

	const Chaos::FSimpleTransmissionConfig& GetPhysicsTransmissionConfig()
	{
		FillTransmissionSetup();
		return PTransmissionConfig;
	}

	void InitDefaults()
	{
		bUseAutomaticGears = true;
		bUseAutoReverse = true;
		FinalRatio = 3.08f;

		ForwardGearRatios.Add(2.85f);
		ForwardGearRatios.Add(2.02f);
		ForwardGearRatios.Add(1.35f);
		ForwardGearRatios.Add(1.0f);

		ReverseGearRatios.Add(2.86f);

		ChangeUpRPM = 4500.0f;
		ChangeDownRPM = 2000.0f;
		GearChangeTime = 0.4f;

		TransmissionEfficiency = 0.9f;
	}

	float GetGearRatio(int32 InGear)
	{
		if (InGear > 0) // a forwards gear
		{
			return ForwardGearRatios[InGear - 1] * FinalRatio;
		}
		else if (InGear < 0) // a reverse gear
		{
			return -ReverseGearRatios[FMath::Abs(InGear) - 1] * FinalRatio;
		}
		else
		{
			return 0.f; // neutral has no ratio
		}
	}


private:

	void FillTransmissionSetup()
	{
		PTransmissionConfig.TransmissionType = this->bUseAutomaticGears ? Chaos::ETransmissionType::Automatic : Chaos::ETransmissionType::Manual;
		PTransmissionConfig.AutoReverse = this->bUseAutoReverse;
		PTransmissionConfig.ChangeUpRPM = this->ChangeUpRPM;
		PTransmissionConfig.ChangeDownRPM = this->ChangeDownRPM;
		PTransmissionConfig.GearChangeTime = this->GearChangeTime;
		PTransmissionConfig.FinalDriveRatio = this->FinalRatio;
		PTransmissionConfig.ForwardRatios.Reset();
		PTransmissionConfig.TransmissionEfficiency = this->TransmissionEfficiency;
		for (float Ratio : this->ForwardGearRatios)
		{
			PTransmissionConfig.ForwardRatios.Add(Ratio);
		}

		PTransmissionConfig.ReverseRatios.Reset();
		for (float Ratio : this->ReverseGearRatios)
		{
			PTransmissionConfig.ReverseRatios.Add(Ratio);
		}
	}

	Chaos::FSimpleTransmissionConfig PTransmissionConfig;

};

/** Single angle : both wheels steer by the same amount
 *  AngleRatio   : outer wheels on corner steer less than the inner ones by set ratio
 *  Ackermann	 : Ackermann steering principle is applied */
UENUM()
enum class ESteeringType : uint8
{
	SingleAngle,
	AngleRatio,
	Ackermann,
};


USTRUCT()
struct CHAOSVEHICLES_API FVehicleSteeringConfig
{
	GENERATED_USTRUCT_BODY()

	FVehicleSteeringConfig()
	{
		InitDefaults();
	}

	/** Single angle : both wheels steer by the same amount
	 *  AngleRatio   : outer wheels on corner steer less than the inner ones by set ratio 
	 *  Ackermann	 : Ackermann steering principle is applied */
	UPROPERTY(EditAnywhere, Category = SteeringSetup)
	ESteeringType SteeringType;

	/** Only applies when AngleRatio is selected */
	UPROPERTY(EditAnywhere, Category = SteeringSetup)
	float AngleRatio; 

	/** Maximum steering versus forward speed (MPH) */
	UPROPERTY(EditAnywhere, Category = SteeringSetup)
	FRuntimeFloatCurve SteeringCurve;


	const Chaos::FSimpleSteeringConfig& GetPhysicsSteeringConfig(FVector2D WheelTrackDimensions)
	{
		FillSteeringSetup(WheelTrackDimensions);
		return PSteeringConfig;
	}

	void InitDefaults()
	{
		SteeringType = ESteeringType::AngleRatio;
		AngleRatio = 0.7f;

		// Init steering speed curve
		FRichCurve* SteeringCurveData = SteeringCurve.GetRichCurve();
		SteeringCurveData->AddKey(0.f, 1.f);
		SteeringCurveData->AddKey(20.f, 0.8f);
		SteeringCurveData->AddKey(60.f, 0.4f);
		SteeringCurveData->AddKey(120.f, 0.3f);
	}

private:

	void FillSteeringSetup(FVector2D WheelTrackDimensions)
	{

		PSteeringConfig.SteeringType = (Chaos::ESteerType)this->SteeringType;
		PSteeringConfig.AngleRatio = AngleRatio;

		float MinValue = 0.f, MaxValue = 1.f;
		this->SteeringCurve.GetRichCurveConst()->GetValueRange(MinValue, MaxValue);
		float MaxX = this->SteeringCurve.GetRichCurveConst()->GetLastKey().Time;
		PSteeringConfig.SpeedVsSteeringCurve.Empty();
		float NumSamples = 20;
		for (float X = 0; X <= MaxX; X += (MaxX / NumSamples))
		{
			float Y = this->SteeringCurve.GetRichCurveConst()->Eval(X) / MaxValue;
			PSteeringConfig.SpeedVsSteeringCurve.Add(FVector2D(X, Y));
		}

		PSteeringConfig.TrackWidth = WheelTrackDimensions.Y;
		PSteeringConfig.WheelBase = WheelTrackDimensions.X;
	}

	Chaos::FSimpleSteeringConfig PSteeringConfig;

};


USTRUCT()
struct CHAOSVEHICLES_API FChaosWheelSetup
{
	GENERATED_USTRUCT_BODY()

	// The wheel class to use
	UPROPERTY(EditAnywhere, Category = WheelSetup)
	TSubclassOf<UChaosVehicleWheel> WheelClass;

	// Bone name on mesh to create wheel at
	//UPROPERTY(EditAnywhere, Category = WheelSetup)
	//FName SteeringBoneName;

	// Bone name on mesh to create wheel at
	UPROPERTY(EditAnywhere, Category = WheelSetup)
	FName BoneName;

	// Additional offset to give the wheels for this axle.
	UPROPERTY(EditAnywhere, Category = WheelSetup)
	FVector AdditionalOffset;

	FChaosWheelSetup();
};

/** Commonly used Wheel state - evaluated once used wherever required for that frame */
struct CHAOSVEHICLES_API FWheelState
{
	FWheelState();
	~FWheelState();

	FWheelState(const FWheelState& Other);
	FWheelState& operator=(const FWheelState& Other);

	void Init(int NumWheels);

	/** Commonly used Wheel state - evaluated once used wherever required for that frame */
	void CaptureState(int WheelIdx, const FVector& WheelOffset, const FBodyInstance* TargetInstance);
	void CaptureState(int WheelIdx, const FVector& WheelOffset, const Chaos::FRigidBodyHandle_Internal* Handle);
	void CaptureState(int WheelIdx, const FVector& WheelOffset, const Chaos::FRigidBodyHandle_Internal* VehicleHandle, const FVector& ContactPoint, const Chaos::FRigidBodyHandle_Internal* SurfaceHandle);
	static FVector GetVelocityAtPoint(const Chaos::FRigidBodyHandle_Internal* Rigid, const FVector& InPoint);

	TArray<FVector> WheelLocalLocation;	/** Current Location Of Wheels In Local Coordinates */
	TArray<FVector> WheelWorldLocation;	/** Current Location Of Wheels In World Coordinates */
	TArray<FVector> WorldWheelVelocity; /** Current velocity at wheel location In World Coordinates - combined linear and angular */
	TArray<FVector> LocalWheelVelocity; /** Local velocity of Wheel */
	TArray<Chaos::FSuspensionTrace> Trace;
	TArray<FHitResult> TraceResult;
};

//////////////////////////////////////////////////////////////////////////

class CHAOSVEHICLES_API UChaosWheeledVehicleSimulation : public UChaosVehicleSimulation
{
public:

	UChaosWheeledVehicleSimulation();

	virtual ~UChaosWheeledVehicleSimulation();

	virtual void Init(TUniquePtr<Chaos::FSimpleWheeledVehicle>& PVehicleIn) override
	{
		UChaosVehicleSimulation::Init(PVehicleIn);

		WheelState.Init(PVehicle->Wheels.Num());
	}

	virtual void UpdateConstraintHandles(TArray<FPhysicsConstraintHandle>& ConstraintHandlesIn) override;

	virtual void TickVehicle(UWorld* WorldIn, float DeltaTime, const FChaosVehicleAsyncInput& InputData, FChaosVehicleAsyncOutput& OutputData, Chaos::FRigidBodyHandle_Internal* Handle) override;

	/** Advance the vehicle simulation */
	virtual void UpdateSimulation(float DeltaTime, const FChaosVehicleAsyncInput& InputData, Chaos::FRigidBodyHandle_Internal* Handle) override;

	/** Update the vehicle state */
	virtual void UpdateState(float DeltaTime, const FChaosVehicleAsyncInput& InputData, Chaos::FRigidBodyHandle_Internal* Handle) override;

	virtual void FillOutputState(FChaosVehicleAsyncOutput& Output) override;

	/** Are enough vehicle systems specified such that physics vehicle simulation is possible */
	virtual bool CanSimulate() const override;

	/** Pass control Input to the vehicle systems */
	virtual void ApplyInput(const FControlInputs& ControlInputs, float DeltaTime) override;

	/** Perform suspension ray/shape traces */
	virtual void PerformSuspensionTraces(const TArray<Chaos::FSuspensionTrace>& SuspensionTrace, FCollisionQueryParams& TraceParams, FCollisionResponseContainer& CollisionResponse, TArray<FWheelTraceParams>& WheelTraceParams);


	/** Update the engine/transmission simulation */
	virtual void ProcessMechanicalSimulation(float DeltaTime);

	/** Process steering mechanism */
	virtual void ProcessSteering(const FControlInputs& ControlInputs);

	/** calculate and apply lateral and longitudinal friction forces from wheels */
	virtual void ApplyWheelFrictionForces(float DeltaTime);

	/** calculate and apply chassis suspension forces */
	virtual void ApplySuspensionForces(float DeltaTime, TArray<FWheelTraceParams>& WheelTraceParams);

	bool IsWheelSpinning() const;
	bool ContainsTraces(const FBox& Box, const TArray<struct Chaos::FSuspensionTrace>& SuspensionTrace);


	/** Draw 3D debug lines and things along side the 3D model */
	virtual void DrawDebug3D() override;

	FWheelState WheelState;	/** Cached state that holds wheel data for this frame */

	TArray<FPhysicsConstraintHandle> ConstraintHandles;

	// cache trace overlap query
	TArray<FOverlapResult> OverlapResults;
	bool bOverlapHit;
	FBox QueryBox;
};

//////////////////////////////////////////////////////////////////////////

UCLASS(ClassGroup = (Physics), meta = (BlueprintSpawnableComponent), hidecategories = (PlanarMovement, "Components|Movement|Planar", Activation, "Components|Activation"))
class CHAOSVEHICLES_API UChaosWheeledVehicleMovementComponent : public UChaosVehicleMovementComponent
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category = WheelSetup)
	bool bSuspensionEnabled;

	UPROPERTY(EditAnywhere, Category = WheelSetup)
	bool bWheelFrictionEnabled;

	UPROPERTY(EditAnywhere, Category = WheelSetup)
	bool bLegacyWheelFrictionPosition;

	/** Wheels to create */
	UPROPERTY(EditAnywhere, Category = WheelSetup)
	TArray<FChaosWheelSetup> WheelSetups;

	UPROPERTY(EditAnywhere, Category = Custom)
	struct FCollisionResponseContainer WheelTraceCollisionResponses;

	UPROPERTY(EditAnywhere, Category = MechanicalSetup)
	bool bMechanicalSimEnabled;

	/** Engine */
	UPROPERTY(EditAnywhere, Category = MechanicalSetup, meta = (EditCondition = "bMechanicalSimEnabled"))
	FVehicleEngineConfig EngineSetup;

	/** Differential */
	UPROPERTY(EditAnywhere, Category = MechanicalSetup, meta = (EditCondition = "bMechanicalSimEnabled"))
	FVehicleDifferentialConfig DifferentialSetup;

	/** Transmission data */
	UPROPERTY(EditAnywhere, Category = MechanicalSetup, meta = (EditCondition = "bMechanicalSimEnabled"))
	FVehicleTransmissionConfig TransmissionSetup;

	/** Transmission data */
	UPROPERTY(EditAnywhere, Category = SteeringSetup)
	FVehicleSteeringConfig SteeringSetup;

	// Our instanced wheels
	UPROPERTY(transient, duplicatetransient, BlueprintReadOnly, Category = Vehicle)
	TArray<TObjectPtr<class UChaosVehicleWheel>> Wheels;

	/** Get current engine's rotation speed */
	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosWheeledVehicleMovement")
	float GetEngineRotationSpeed() const;

	/** Get current engine's max rotation speed */
	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosWheeledVehicleMovement")
	float GetEngineMaxRotationSpeed() const;

	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosWheeledVehicleMovement")
	int GetNumWheels() const 
	{
		return WheelStatus.Num();
	}

	UFUNCTION(BlueprintPure, Category = "Vehicles")
	static void BreakWheelStatus(const struct FWheelStatus& Status, bool& bInContact, FVector& ContactPoint, UPhysicalMaterial*& PhysMaterial
			, float& NormalizedSuspensionLength, float& SpringForce, float& SlipAngle, bool& bIsSlipping, float& SlipMagnitude, bool& bIsSkidding, float& SkidMagnitude, FVector& SkidNormal, float& DriveTorque, float& BrakeTorque, bool& bABSActivated);

	UFUNCTION(BlueprintPure, Category = "Vehicles")
	static FWheelStatus MakeWheelStatus(bool bInContact, FVector& ContactPoint, UPhysicalMaterial* PhysMaterial
			, float NormalizedSuspensionLength, float SpringForce, float SlipAngle, bool bIsSlipping, float SlipMagnitude, bool bIsSkidding, float SkidMagnitude, FVector& SkidNormal, float DriveTorque, float BrakeTorque, bool bABSActivated);

	UFUNCTION(BlueprintPure, Category = "Vehicles")
	static void BreakWheeledSnapshot(const struct FWheeledSnaphotData& Snapshot, FTransform& Transform, FVector& LinearVelocity
			, FVector& AngularVelocity, int& SelectedGear, float& EngineRPM, TArray<FWheelSnapshot>& WheelSnapshots);

	UFUNCTION(BlueprintPure, Category = "Vehicles")
	static FWheeledSnaphotData MakeWheeledSnapshot(FTransform Transform, FVector LinearVelocity, FVector AngularVelocity
			, int SelectedGear, float EngineRPM, const TArray<FWheelSnapshot>& WheelSnapshots);

	UFUNCTION(BlueprintPure, Category = "Vehicles")
	static void BreakWheelSnapshot(const struct FWheelSnapshot& Snapshot, float& SuspensionOffset
			, float& WheelRotationAngle, float& SteeringAngle, float& WheelRadius, float& WheelAngularVelocity);

	UFUNCTION(BlueprintPure, Category = "Vehicles")
	static FWheelSnapshot MakeWheelSnapshot(float SuspensionOffset, float WheelRotationAngle
			, float SteeringAngle, float WheelRadius, float WheelAngularVelocity);

	/** Get a wheels current simulation state */
	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosWheeledVehicleMovement")
	const FWheelStatus& GetWheelState(int WheelIndex) const
	{
		return WheelStatus[WheelIndex];
	}

	virtual float GetSuspensionOffset(int WheelIndex) override;
	UPhysicalMaterial* GetPhysMaterial(int WheelIndex);

	/** Set all channels to the specified response - for wheel raycasts */
	void SetWheelTraceAllChannels(ECollisionResponse NewResponse)
	{
		WheelTraceCollisionResponses.SetAllChannels(NewResponse);
	}

	/** Set the response of this body to the supplied settings - for wheel raycasts */
	void SetWheelTraceResponseToChannel(ECollisionChannel Channel, ECollisionResponse NewResponse)
	{
		WheelTraceCollisionResponses.SetResponse(Channel, NewResponse);
	}

	/** Get Collision ResponseToChannels container for this component **/
//	FORCEINLINE_DEBUGGABLE const FCollisionResponseContainer& GetTraceResponseToChannels() const { return WheelTraceCollisionResponses.GetResponseContainer(); }

	//////////////////////////////////////////////////////////////////////////
	// Public

	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;

	// Get output data from Physics Thread
	virtual void ParallelUpdate(float DeltaSeconds);

#if WITH_EDITOR
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	/** Are the configuration references configured sufficiently that the vehicle can be created */
	virtual bool CanCreateVehicle() const override;

	/** Used to create any physics engine information for this component */
	virtual void OnCreatePhysicsState() override;

	/** Used to shut down and pysics engine structure for this component */
	virtual void OnDestroyPhysicsState() override;

	/** display next debug page */
	static void NextDebugPage();

	/** display previous debug page */
	static void PrevDebugPage();

	/** Enable or completely bypass the ProcessMechanicalSimulation call */
	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosWheeledVehicleMovement")
	void EnableMechanicalSim(bool InState)
	{
		bMechanicalSimEnabled = InState;
	}

	/** Enable or completely bypass the ApplySuspensionForces call */
	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosWheeledVehicleMovement")
	void EnableSuspension(bool InState)
	{
		bSuspensionEnabled = InState;
	}

	/** Enable or completely bypass the ApplyWheelFrictionForces call */
	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosWheeledVehicleMovement")
	void EnableWheelFriction(bool InState)
	{
		bWheelFrictionEnabled = InState;
	}

	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosWheeledVehicleMovement")
	void SetWheelClass(int WheelIndex, TSubclassOf<UChaosVehicleWheel> InWheelClass);

	/** Grab a snapshot of the vehicle instance dynamic state */
	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosWheeledVehicleMovement")
	virtual FWheeledSnaphotData GetSnapshot() const;

	/** Set snapshot of vehicle instance dynamic state */
	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosWheeledVehicleMovement")
	virtual void SetSnapshot(const FWheeledSnaphotData& SnapshotIn);


	//////////////////////////////////////////////////////////////////////////
	// change handling via blueprint at runtime
	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosWheeledVehicleMovement")
	void SetMaxEngineTorque(float Torque);

	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosWheeledVehicleMovement")
	void SetDragCoefficient(float DragCoeff);

	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosWheeledVehicleMovement")
	void SetDownforceCoefficient(float DownforceCoeff);

	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosWheeledVehicleMovement")
	void SetDifferentialFrontRearSplit(float FrontRearSlpit);

	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosWheeledVehicleMovement")
	void SetTractionControlEnabled(int WheelIndex, bool Enabled);

	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosWheeledVehicleMovement")
	void SetABSEnabled(int WheelIndex, bool Enabled);

	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosWheeledVehicleMovement")
	void SetAffectedByBrake(int WheelIndex, bool Enabled);

	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosWheeledVehicleMovement")
	void SetAffectedByHandbrake(int WheelIndex, bool Enabled);

	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosWheeledVehicleMovement")
	void SetAffectedBySteering(int WheelIndex, bool Enabled);

	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosWheeledVehicleMovement")
	void SetAffectedByEngine(int WheelIndex, bool Enabled);

	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosWheeledVehicleMovement")
	void SetWheelRadius(int WheelIndex, float Radius);

	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosWheeledVehicleMovement")
	void SetWheelFrictionMultiplier(int WheelIndex, float Friction);

	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosWheeledVehicleMovement")
	void SetWheelSlipGraphMultiplier(int WheelIndex, float Multiplier);

	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosWheeledVehicleMovement")
	void SetWheelMaxBrakeTorque(int WheelIndex, float Torque);

	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosWheeledVehicleMovement")
	void SetWheelHandbrakeTorque(int WheelIndex, float Torque);

	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosWheeledVehicleMovement")
	void SetWheelMaxSteerAngle(int WheelIndex, float AngleDegrees);

	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosWheeledVehicleMovement")
	void SetTorqueCombineMethod(ETorqueCombineMethod InCombineMethod, int32 WheelIndex);

	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosWheeledVehicleMovement")
	void SetDriveTorque(float DriveTorque, int32 WheelIndex);
	
	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosWheeledVehicleMovement")
	void SetBrakeTorque(float BrakeTorque, int32 WheelIndex);

	UFUNCTION(BlueprintCallable, Category = "Game|Components|ChaosWheeledVehicleMovement")
	void SetSuspensionParams(float Rate, float Damping, float Preload, float MaxRaise, float MaxDrop, int32 WheelIndex);

	/** */
	virtual TUniquePtr<Chaos::FSimpleWheeledVehicle> CreatePhysicsVehicle() override
	{
		// Make the Vehicle Simulation class that will be updated from the physics thread async callback
		VehicleSimulationPT = MakeUnique<UChaosWheeledVehicleSimulation>();

		return UChaosVehicleMovementComponent::CreatePhysicsVehicle();
	}

	/** Allocate and setup the Chaos vehicle */
	virtual void SetupVehicle(TUniquePtr<Chaos::FSimpleWheeledVehicle>& PVehicle) override;

	virtual void ResetVehicleState() override;

protected:

	//////////////////////////////////////////////////////////////////////////
	// Setup

	/** Re-Compute any runtime constants values that rely on setup data */
	virtual void ComputeConstants() override;

	/** Skeletal mesh needs some special handling in the vehicle case */
	virtual void FixupSkeletalMesh();

	/** Create and setup the Chaos vehicle */
	virtual void CreateVehicle();

	/** Instantiate and setup our wheel objects */
	virtual void CreateWheels();

	/** Release our wheel objects */
	virtual void DestroyWheels();

	/** Set up the chassis and wheel shapes */
	virtual void SetupVehicleShapes();

	/** Setup calculated suspension parameters */
	void SetupSuspension(TUniquePtr<Chaos::FSimpleWheeledVehicle>& PVehicle);

	/** Maps UChaosVehicleWheel Axle to a wheel index */
	void RecalculateAxles();

	/** Get the local position of the wheel at rest */
	virtual FVector GetWheelRestingPosition(const FChaosWheelSetup& WheelSetup);

	//////////////////////////////////////////////////////////////////////////
	// Update
	void FillWheelOutputState();

	/* Fill Async input state */
	virtual void Update(float DeltaTime) override;

	//////////////////////////////////////////////////////////////////////////
	// Debug

	/** Draw 2D debug text graphs on UI for the wheels, suspension and other systems */
	virtual void DrawDebug(UCanvas* Canvas, float& YL, float& YPos);

	/** Get distances between wheels - primarily a debug display helper */
	const FVector2D& GetWheelLayoutDimensions() const
	{
		return WheelTrackDimensions;
	}

#if UE_WITH_REMOTE_OBJECT_HANDLE
	ENGINE_API virtual void PostMigrate(const struct FUObjectMigrationContext& MigrationContext) override;
#endif

	private:


	/** Get distances between wheels - primarily a debug display helper */
	FVector2D CalculateWheelLayoutDimensions();

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	float CalcDialAngle(float CurrentValue, float MaxValue);
	void DrawDial(UCanvas* Canvas, FVector2D Pos, float Radius, float CurrentValue, float MaxValue);
#endif

	struct FCachedState
	{
		FCachedState() : WheelOffset(0.f), bIsValid(false)
		{ }

		float WheelOffset;
		bool bIsValid;
	};

	static EDebugPages DebugPage;
	uint32 NumDrivenWheels; /** The number of wheels that have engine enabled checked */
	FVector2D WheelTrackDimensions;	// Wheelbase (X) and track (Y) dimensions
	TMap<UChaosVehicleWheel*, TArray<int>> AxleToWheelMap;
	TArray<FPhysicsConstraintHandle> ConstraintHandles;
	TArray<FWheelStatus> WheelStatus; /** Wheel output status */
	TArray<FCachedState> CachedState;
	Chaos::FPerformanceMeasure PerformanceMeasure;

#if UE_WITH_REMOTE_OBJECT_HANDLE
	bool bAreWheelsMigrated = false;
#endif

};

#if VEHICLE_DEBUGGING_ENABLED
UE_ENABLE_OPTIMIZATION
#endif
