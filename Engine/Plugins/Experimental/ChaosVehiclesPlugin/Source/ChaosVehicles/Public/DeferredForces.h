// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"


enum class EForceFlags : uint32
{
	None = 0, // No flags.

	AllowSubstepping = 1 << 0,
	AccelChange = 1 << 1,
	VelChange = 1 << 2,
	IsLocalForce = 1 << 3
};
ENUM_CLASS_FLAGS(EForceFlags);	


// Forward declarations
namespace Chaos
{ 
	class FRigidBodyHandle_Internal;
}


class CHAOSVEHICLES_API FDeferredForces
{
public:



	struct FApplyForceData
	{


		FApplyForceData(const FVector& ForceIn, bool bAllowSubsteppingIn, bool bAccelChangeIn)
			: Force(ForceIn)
			, Flags(EForceFlags::None)
		{
			Flags |= bAllowSubsteppingIn ? EForceFlags::AllowSubstepping : EForceFlags::None;
			Flags |= bAccelChangeIn ? EForceFlags::AccelChange : EForceFlags::None;
		}

		FVector Force;
		EForceFlags Flags;
	};

	struct FApplyForceAtPositionData
	{
		FApplyForceAtPositionData(const FVector& ForceIn, const FVector& PositionIn, bool bAllowSubsteppingIn, bool bIsLocalForceIn)
			: Force(ForceIn)
			, Position(PositionIn)
			, Flags(EForceFlags::None)
		{
			Flags |= bAllowSubsteppingIn ? EForceFlags::AllowSubstepping : EForceFlags::None;
			Flags |= bIsLocalForceIn ? EForceFlags::IsLocalForce : EForceFlags::None;
		}

		FVector Force;
		FVector Position;
		EForceFlags Flags;
	};

	struct FAddTorqueInRadiansData
	{
		FAddTorqueInRadiansData(const FVector& TorqueIn, bool bAllowSubsteppingIn, bool bAccelChangeIn)
			: Torque(TorqueIn)
			, Flags(EForceFlags::None)
		{
			Flags |= bAllowSubsteppingIn ? EForceFlags::AllowSubstepping : EForceFlags::None;
			Flags |= bAccelChangeIn ? EForceFlags::AccelChange : EForceFlags::None;
		}

		FVector Torque;
		EForceFlags Flags;
	};

	struct FAddImpulseData
	{
		FAddImpulseData(const FVector& ImpulseIn, const bool bVelChangeIn)
			: Impulse(ImpulseIn)
			, Flags(EForceFlags::None)
		{
			Flags |= bVelChangeIn ? EForceFlags::VelChange : EForceFlags::None;
		}

		FVector Impulse;
		EForceFlags Flags;
	};

	struct FAddImpulseAtPositionData
	{
		FAddImpulseAtPositionData(const FVector& ImpulseIn, const FVector& PositionIn)
			: Impulse(ImpulseIn)
			, Position(PositionIn)
		{

		}

		FVector Impulse;
		FVector Position;
	};

	void Add(const FApplyForceData& ApplyForceDataIn)
	{
		ApplyForceDatas.Add(ApplyForceDataIn);
	}

	void Add(const FApplyForceAtPositionData& ApplyForceAtPositionDataIn)
	{
		ApplyForceAtPositionDatas.Add(ApplyForceAtPositionDataIn);
	}

	void Add(const FAddTorqueInRadiansData& ApplyTorqueDataIn)
	{
		ApplyTorqueDatas.Add(ApplyTorqueDataIn);
	}

	void Add(const FAddImpulseData& ApplyImpulseDataIn)
	{
		ApplyImpulseDatas.Add(ApplyImpulseDataIn);
	}

	void Add(const FAddImpulseAtPositionData& ApplyImpulseAtPositionDataIn)
	{
		ApplyImpulseAtPositionDatas.Add(ApplyImpulseAtPositionDataIn);
	}


	void Apply(Chaos::FRigidBodyHandle_Internal* RigidHandle)
	{
		for (const FApplyForceData& Data : ApplyForceDatas)
		{
			AddForce(RigidHandle, Data);
		}

		for (const FApplyForceAtPositionData& Data : ApplyForceAtPositionDatas)
		{
			AddForceAtPosition(RigidHandle, Data);
		}

		for (const FAddTorqueInRadiansData& Data : ApplyTorqueDatas)
		{
			AddTorque(RigidHandle, Data);
		}

		for (const FAddImpulseData& Data : ApplyImpulseDatas)
		{
			AddImpulse(RigidHandle, Data);
		}

		for (const FAddImpulseAtPositionData& Data : ApplyImpulseAtPositionDatas)
		{
			AddImpulseAtPosition(RigidHandle, Data);
		}

		ApplyForceDatas.Empty();
		ApplyForceAtPositionDatas.Empty();
		ApplyTorqueDatas.Empty();
		ApplyImpulseDatas.Empty();
		ApplyImpulseAtPositionDatas.Empty();
	}

private:

	void AddForce(Chaos::FRigidBodyHandle_Internal* RigidHandle, const FApplyForceData& DataIn);
	void AddForceAtPosition(Chaos::FRigidBodyHandle_Internal* RigidHandle, const FApplyForceAtPositionData& DataIn);
	void AddTorque(Chaos::FRigidBodyHandle_Internal* RigidHandle, const FAddTorqueInRadiansData& DataIn);
	void AddImpulse(Chaos::FRigidBodyHandle_Internal* RigidHandle, const FAddImpulseData& DataIn);
	void AddImpulseAtPosition(Chaos::FRigidBodyHandle_Internal* RigidHandle, const FAddImpulseAtPositionData& DataIn);

	TArray<FApplyForceData> ApplyForceDatas;
	TArray<FApplyForceAtPositionData> ApplyForceAtPositionDatas;
	TArray<FAddTorqueInRadiansData> ApplyTorqueDatas;
	TArray<FAddImpulseData> ApplyImpulseDatas;
	TArray<FAddImpulseAtPositionData> ApplyImpulseAtPositionDatas;
};
