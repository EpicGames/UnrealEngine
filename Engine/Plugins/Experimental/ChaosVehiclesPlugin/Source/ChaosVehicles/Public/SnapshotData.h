// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SnapshotData.generated.h"


USTRUCT(BlueprintType)
struct CHAOSVEHICLES_API FBaseSnapshotData //: public UObject
{
public:
	GENERATED_BODY()

	FBaseSnapshotData()
	{
		Transform = FTransform::Identity;
		LinearVelocity = FVector::ZeroVector;
		AngularVelocity = FVector::ZeroVector;
	}

	UPROPERTY()
	FTransform Transform;		// world coords

	UPROPERTY()
	FVector LinearVelocity;		// world coords

	UPROPERTY()
	FVector AngularVelocity;	// world coords

};

USTRUCT(BlueprintType)
struct CHAOSVEHICLES_API FWheelSnapshot
{
	GENERATED_BODY()

	FWheelSnapshot()
	{
		SuspensionOffset = 0.f;
		WheelRotationAngle = 0.f;
		SteeringAngle = 0.f;
		WheelRadius = 0.f;
		WheelAngularVelocity = 0.f;
	}
	
	UPROPERTY()
	float SuspensionOffset;		// suspension location
	
	UPROPERTY()
	float WheelRotationAngle;	// wheel rotation angle, rotated position
	
	UPROPERTY()
	float SteeringAngle;		// steering position

	UPROPERTY()
	float WheelRadius;			// radius of the wheel can be changed dynamically, to sim damaged ot flat

	UPROPERTY()
	float WheelAngularVelocity;	// speed of rotation of wheel
};

USTRUCT(BlueprintType)
struct CHAOSVEHICLES_API FWheeledSnaphotData : public FBaseSnapshotData
{
public:
	GENERATED_BODY()

	FWheeledSnaphotData()
	{
		SelectedGear = 0;
		EngineRPM = 0.f;
	}

	UPROPERTY()
	int SelectedGear;		// -ve reverse gear(s), 0 neutral, +ve forward gears

	UPROPERTY()
	float EngineRPM;		// Engine Revolutions Per Minute

	UPROPERTY()
	TArray<FWheelSnapshot> WheelSnapshots;

};
