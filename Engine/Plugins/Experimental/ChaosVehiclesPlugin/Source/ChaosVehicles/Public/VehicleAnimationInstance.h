// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimInstanceProxy.h"
#include "VehicleAnimationInstance.generated.h"

class UChaosWheeledVehicleMovementComponent;

struct FWheelAnimationData
{
	FName BoneName;
	FRotator RotOffset;
	FVector LocOffset;
};

/** Proxy override for this UAnimInstance-derived class */
USTRUCT()
	struct CHAOSVEHICLES_API FVehicleAnimationInstanceProxy : public FAnimInstanceProxy
{
	GENERATED_BODY()

		FVehicleAnimationInstanceProxy()
		: FAnimInstanceProxy()
		, WheelSpokeCount(0)
		, MaxAngularVelocity(256.f)
		, ShutterSpeed(30.f)
		, StageCoachBlend(730.f)
	{
	}

	FVehicleAnimationInstanceProxy(UAnimInstance* Instance)
		: FAnimInstanceProxy(Instance)
		, WheelSpokeCount(0)
		, MaxAngularVelocity(256.f)
		, ShutterSpeed(30.f)
		, StageCoachBlend(730.f)
	{
	}

public:

	void SetWheeledVehicleComponent(const UChaosWheeledVehicleMovementComponent* InWheeledVehicleComponent);

	/** FAnimInstanceProxy interface begin*/
	virtual void PreUpdate(UAnimInstance* InAnimInstance, float DeltaSeconds) override;
	/** FAnimInstanceProxy interface end*/

	const TArray<FWheelAnimationData>& GetWheelAnimData() const
	{
		return WheelInstances;
	}

	void SetStageCoachEffectParams(int InWheelSpokeCount, float InMaxAngularVelocity, float InShutterSpeed, float InStageCoachBlend)
	{
		WheelSpokeCount = InWheelSpokeCount;
		MaxAngularVelocity = InMaxAngularVelocity;
		ShutterSpeed = InShutterSpeed;
		StageCoachBlend = InStageCoachBlend;
	}

private:
	TArray<FWheelAnimationData> WheelInstances;

	int WheelSpokeCount;			// Number of spokes visible on wheel
	float MaxAngularVelocity;		// Wheel max rotation speed in degrees/second
	float ShutterSpeed;				// Camera shutter speed in frames/second
	float StageCoachBlend;			// Blend effect degrees/second

};

UCLASS(transient)
	class CHAOSVEHICLES_API UVehicleAnimationInstance : public UAnimInstance
{
	GENERATED_UCLASS_BODY()

		/** Makes a montage jump to the end of a named section. */
		UFUNCTION(BlueprintCallable, Category = "Animation")
		class AWheeledVehiclePawn* GetVehicle();

public:
	TArray<TArray<FWheelAnimationData>> WheelData;

public:
	void SetWheeledVehicleComponent(const UChaosWheeledVehicleMovementComponent* InWheeledVehicleComponent)
	{
		WheeledVehicleComponent = InWheeledVehicleComponent;
		AnimInstanceProxy.SetWheeledVehicleComponent(InWheeledVehicleComponent);
	}

	const UChaosWheeledVehicleMovementComponent* GetWheeledVehicleComponent() const
	{
		return WheeledVehicleComponent;
	}

private:
	/** UAnimInstance interface begin*/
	virtual void NativeInitializeAnimation() override;
	virtual FAnimInstanceProxy* CreateAnimInstanceProxy() override;
	virtual void DestroyAnimInstanceProxy(FAnimInstanceProxy* InProxy) override;
	/** UAnimInstance interface end*/

	FVehicleAnimationInstanceProxy AnimInstanceProxy;

	UPROPERTY(transient)
	TObjectPtr<const UChaosWheeledVehicleMovementComponent> WheeledVehicleComponent;
};


