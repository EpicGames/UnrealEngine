// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Pawn.h"
#include "WheeledVehiclePawn.generated.h"

class FDebugDisplayInfo;

/**
 * ChaosWheeledVehicle is the base wheeled vehicle pawn actor.
 * By default it uses UChaosWheeledVehicleMovementComponent for its simulation, but this can be overridden by inheriting from the class and modifying its constructor like so:
 * Super(ObjectInitializer.SetDefaultSubobjectClass<UMyMovement>(VehicleComponentName))
 * Where UMyMovement is the new movement type that inherits from UChaosVehicleMovementComponent
 */

UCLASS(abstract, config=Game, BlueprintType)
class CHAOSVEHICLES_API AWheeledVehiclePawn : public APawn
{
	GENERATED_UCLASS_BODY()

private:
	/**  The main skeletal mesh associated with this Vehicle */
	UPROPERTY(Category = Vehicle, VisibleDefaultsOnly, BlueprintReadOnly, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<class USkeletalMeshComponent> Mesh;

	/** vehicle simulation component */
	UPROPERTY(Category = Vehicle, VisibleDefaultsOnly, BlueprintReadOnly, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<class UChaosVehicleMovementComponent> VehicleMovementComponent;
public:

	/** Name of the MeshComponent. Use this name if you want to prevent creation of the component (with ObjectInitializer.DoNotCreateDefaultSubobject). */
	static FName VehicleMeshComponentName;

	/** Name of the VehicleMovement. Use this name if you want to use a different class (with ObjectInitializer.SetDefaultSubobjectClass). */
	static FName VehicleMovementComponentName;

	/** Util to get the wheeled vehicle movement component */
	class UChaosVehicleMovementComponent* GetVehicleMovementComponent() const;

	//~ Begin AActor Interface
	virtual void DisplayDebug(class UCanvas* Canvas, const FDebugDisplayInfo& DebugDisplay, float& YL, float& YPos) override;
	//~ End Actor Interface

	/** Returns Mesh subobject **/
	class USkeletalMeshComponent* GetMesh() const { return Mesh; }

	/** Returns VehicleMovement subobject **/
	class UChaosVehicleMovementComponent* GetVehicleMovement() const { return VehicleMovementComponent; }
};
