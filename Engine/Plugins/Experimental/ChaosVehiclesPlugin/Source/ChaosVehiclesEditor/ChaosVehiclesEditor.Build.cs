// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ChaosVehiclesEditor : ModuleRules
	{
        public ChaosVehiclesEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Slate",
					"SlateCore",
					"Engine",
					"EditorFramework",
					"UnrealEd",
					"PropertyEditor",
					"AnimGraphRuntime",
					"AnimGraph",
					"BlueprintGraph",
					"ToolMenus",
					"PhysicsCore",
					"ChaosVehiclesCore",
					"ChaosVehiclesEngine",
					"ChaosVehicles"
				}
				);

			PrivateDefinitions.Add("CHAOS_INCLUDE_LEVEL_1=1");
		}
	}
}
