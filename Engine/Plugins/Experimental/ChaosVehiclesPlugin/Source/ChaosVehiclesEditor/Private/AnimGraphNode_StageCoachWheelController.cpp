// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_StageCoachWheelController.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_StageCoachWheelController

#define LOCTEXT_NAMESPACE "A3Nodes"

UAnimGraphNode_StageCoachWheelController::UAnimGraphNode_StageCoachWheelController(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FText UAnimGraphNode_StageCoachWheelController::GetControllerDescription() const
{
	return LOCTEXT("AnimGraphNode_StagecoachEffectWheelController", "Stagecoach Effect Wheel Controller for ChaosWheeledVehicle");
}

FText UAnimGraphNode_StageCoachWheelController::GetTooltipText() const
{
	return LOCTEXT("AnimGraphNode_StagecoachEffectWheelController_Tooltip", "This alters the wheel transform based on set up in Wheeled Vehicle. This only works when the owner is ChaosWheeledVehicle.");
}

FText UAnimGraphNode_StageCoachWheelController::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	FText NodeTitle;
	if (TitleType == ENodeTitleType::ListView || TitleType == ENodeTitleType::MenuTitle)
	{
		NodeTitle = GetControllerDescription();
	}
	else
	{
		// we don't have any run-time information, so it's limited to print  
		// anymore than what it is it would be nice to print more data such as 
		// name of bones for wheels, but it's not available in Persona
		NodeTitle = FText(LOCTEXT("AnimGraphNode_StagecoachWheelController_Title", "Stagecoach Effect Wheel Controller"));
	}	
	return NodeTitle;
}

void UAnimGraphNode_StageCoachWheelController::ValidateAnimNodePostCompile(class FCompilerResultsLog& MessageLog, class UAnimBlueprintGeneratedClass* CompiledClass, int32 CompiledNodeIndex)
{
	// we only support vehicle anim instance
	if ( CompiledClass->IsChildOf(UVehicleAnimationInstance::StaticClass())  == false )
	{
		MessageLog.Error(TEXT("@@ is only allowed in VehicleAnimInstance. If this is for vehicle, please change parent to be VehicleAnimInstance (Reparent Class)."), this);
	}
}

bool UAnimGraphNode_StageCoachWheelController::IsCompatibleWithGraph(const UEdGraph* TargetGraph) const
{
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(TargetGraph);
	return (Blueprint != nullptr) && Blueprint->ParentClass->IsChildOf<UVehicleAnimationInstance>() && Super::IsCompatibleWithGraph(TargetGraph);
}

#undef LOCTEXT_NAMESPACE
