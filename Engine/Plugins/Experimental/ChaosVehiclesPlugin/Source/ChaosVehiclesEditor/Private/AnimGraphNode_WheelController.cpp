// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_WheelController.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_WheelController

#define LOCTEXT_NAMESPACE "A3Nodes"

UAnimGraphNode_WheelController::UAnimGraphNode_WheelController(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FText UAnimGraphNode_WheelController::GetControllerDescription() const
{
	return LOCTEXT("AnimGraphNode_WheelController", "Wheel Controller for WheeledVehicle");
}

FText UAnimGraphNode_WheelController::GetTooltipText() const
{
	return LOCTEXT("AnimGraphNode_WheelController_Tooltip", "This alters the wheel transform based on set up in Wheeled Vehicle. This only works when the owner is WheeledVehicle.");
}

FText UAnimGraphNode_WheelController::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	FText NodeTitle;
	if (TitleType == ENodeTitleType::ListView || TitleType == ENodeTitleType::MenuTitle)
	{
		NodeTitle = GetControllerDescription();
	}
	else
	{
		// we don't have any run-time information, so it's limited to print  
		// anymore than what it is it would be nice to print more data such as 
		// name of bones for wheels, but it's not available in Persona
		NodeTitle = FText(LOCTEXT("AnimGraphNode_WheelController_Title", "Wheel Controller"));
	}	
	return NodeTitle;
}

void UAnimGraphNode_WheelController::ValidateAnimNodePostCompile(class FCompilerResultsLog& MessageLog, class UAnimBlueprintGeneratedClass* CompiledClass, int32 CompiledNodeIndex)
{
	// we only support vehicle anim instance
	if ( CompiledClass->IsChildOf(UVehicleAnimationInstance::StaticClass())  == false )
	{
		MessageLog.Error(TEXT("@@ is only allowwed in VehicleAnimInstance. If this is for vehicle, please change parent to be VehicleAnimInstance (Reparent Class)."), this);
	}
}

bool UAnimGraphNode_WheelController::IsCompatibleWithGraph(const UEdGraph* TargetGraph) const
{
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(TargetGraph);
	return (Blueprint != nullptr) && Blueprint->ParentClass->IsChildOf<UVehicleAnimationInstance>() && Super::IsCompatibleWithGraph(TargetGraph);
}

#undef LOCTEXT_NAMESPACE
