// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetTypeActions_ChaosVehicles.h"

#include "ThumbnailRendering/SceneThumbnailInfo.h"
#include "ChaosVehicles.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "AssetTypeActions"

UClass* FAssetTypeActions_ChaosVehicles::GetSupportedClass() const
{
	return UChaosVehicles::StaticClass();
}

UThumbnailInfo* FAssetTypeActions_ChaosVehicles::GetThumbnailInfo(UObject* Asset) const
{
	UChaosVehicles* ChaosVehicles = CastChecked<UChaosVehicles>(Asset);
	return NewObject<USceneThumbnailInfo>(ChaosVehicles, NAME_None, RF_Transactional);
}

void FAssetTypeActions_ChaosVehicles::GetActions(const TArray<UObject*>& InObjects, FToolMenuSection& Section)
{
	FAssetTypeActions_Base::GetActions(InObjects, Section);
}

void FAssetTypeActions_ChaosVehicles::OpenAssetEditor(const TArray<UObject*>& InObjects, TSharedPtr<IToolkitHost> EditWithinLevelEditor)
{
}

#undef LOCTEXT_NAMESPACE
