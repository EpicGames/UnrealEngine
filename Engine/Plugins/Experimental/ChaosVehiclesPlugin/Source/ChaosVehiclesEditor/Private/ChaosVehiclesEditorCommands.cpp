// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVehiclesEditorCommands.h"

#include "Engine/Selection.h"
#include "Engine/World.h"
#include "EngineUtils.h"
#include "ChaosVehicles.h"
#include "Logging/LogMacros.h"
#include "Editor.h"

DEFINE_LOG_CATEGORY_STATIC(UFEC_Log, NoLogging, All);
