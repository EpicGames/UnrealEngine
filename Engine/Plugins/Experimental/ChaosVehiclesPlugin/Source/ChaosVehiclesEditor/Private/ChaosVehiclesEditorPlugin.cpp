// Copyright Epic Games, Inc. All Rights Reserved.


#include "ChaosVehiclesEditorPlugin.h"

#include "AssetToolsModule.h"
#include "CoreMinimal.h"
#include "ChaosVehicles.h"
#include "AssetTypeActions_ChaosVehicles.h"
#include "ChaosVehiclesEditorStyle.h"
#include "ChaosVehiclesEditorCommands.h"
#include "HAL/ConsoleManager.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
//#include "ChaosVehiclesEditorDetails.h"
#include "ChaosVehicleManager.h"

TOptional<FChaosVehiclesEditorStyle> FChaosVehiclesEditorStyle::Singleton;

IMPLEMENT_MODULE( IChaosVehiclesEditorPlugin, ChaosVehiclesEditor )

void IChaosVehiclesEditorPlugin::StartupModule()
{
	//OnUpdatePhysXMaterialHandle = FPhysicsDelegates::OnUpdatePhysXMaterial.AddRaw(this, &FPhysXVehiclesPlugin::UpdatePhysXMaterial);
	//OnPhysicsAssetChangedHandle = FPhysicsDelegates::OnPhysicsAssetChanged.AddRaw(this, &FPhysXVehiclesPlugin::PhysicsAssetChanged);

	FChaosVehiclesEditorStyle::Get();

	FAssetToolsModule& AssetToolsModule = FAssetToolsModule::GetModule();
	IAssetTools& AssetTools = AssetToolsModule.Get();
	AssetTypeActions_ChaosVehicles = new FAssetTypeActions_ChaosVehicles();
	AssetTools.RegisterAssetTypeActions(MakeShareable(AssetTypeActions_ChaosVehicles));

	if (GIsEditor && !IsRunningCommandlet())
	{
	}

	// Register details view customizations
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
}


void IChaosVehiclesEditorPlugin::ShutdownModule()
{
	//FPhysicsDelegates::OnUpdatePhysXMaterial.Remove(OnUpdatePhysXMaterialHandle);
	//FPhysicsDelegates::OnPhysicsAssetChanged.Remove(OnPhysicsAssetChangedHandle);

	if (UObjectInitialized())
	{
		FAssetToolsModule& AssetToolsModule = FAssetToolsModule::GetModule();
		IAssetTools& AssetTools = AssetToolsModule.Get();
		AssetTools.UnregisterAssetTypeActions(AssetTypeActions_ChaosVehicles->AsShared());
	}

	// Unregister details view customizations
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.UnregisterCustomPropertyTypeLayout("ChaosDebugSubstepControl");
}
