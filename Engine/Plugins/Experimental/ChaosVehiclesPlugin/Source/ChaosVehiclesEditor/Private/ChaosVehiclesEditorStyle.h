// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateStyleMacros.h"

class FChaosVehiclesEditorStyle final : public FSlateStyleSet
{
public:
	FChaosVehiclesEditorStyle() : FSlateStyleSet("ChaosVehiclesEditorStyle")
	{
		const FVector2D Icon16x16(16.f, 16.f);
		const FVector2D Icon64x64(64.f, 64.f);

#if !IS_MONOLITHIC
		SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Experimental/ChaosVehiclesPlugin/Resources"));
#endif

		Set("ClassIcon.ChaosVehicles", new IMAGE_BRUSH("ChaosVehicles_16x", Icon16x16));
		Set("ClassThumbnail.ChaosVehicles", new IMAGE_BRUSH("ChaosVehicles_64x", Icon64x64));

		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	~FChaosVehiclesEditorStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}

public:

	static FChaosVehiclesEditorStyle& Get()
	{
		if (!Singleton.IsSet())
		{
			Singleton.Emplace();
		}
		return Singleton.GetValue();
	}

	static void Destroy()
	{
		Singleton.Reset();
	}

private:
	static TOptional<FChaosVehiclesEditorStyle> Singleton;
};
