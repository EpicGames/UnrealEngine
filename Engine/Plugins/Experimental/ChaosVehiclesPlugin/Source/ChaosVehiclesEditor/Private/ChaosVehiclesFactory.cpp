// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVehiclesFactory.h"
//
//#include "Editor.h"
//#include "Editor/EditorEngine.h"
//#include "Engine/Selection.h"
//
//#define LOCTEXT_NAMESPACE "ChaosVehicles"
//
///////////////////////////////////////////////////////
//// ChaosVehiclesFactory
//
//UChaosVehiclesFactory::UChaosVehiclesFactory(const FObjectInitializer& ObjectInitializer)
//	: Super(ObjectInitializer)
//{
//	bCreateNew = true;
//	bEditAfterNew = true;
//	SupportedClass = UChaosVehicles::StaticClass();
//}
//
//UChaosVehicles* UChaosVehiclesFactory::StaticFactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
//{
//	return static_cast<UChaosVehicles*>(NewObject<UChaosVehicles>(InParent, Class, Name, Flags | RF_Transactional | RF_Public | RF_Standalone));
//}
//
//UObject* UChaosVehiclesFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
//{
//	UChaosVehicles* NewChaosVehicles = StaticFactoryCreateNew(Class, InParent, Name, Flags, Context, Warn);
//	NewChaosVehicles->MarkPackageDirty();
//	return NewChaosVehicles;
//}
//
//#undef LOCTEXT_NAMESPACE
//
//
//
