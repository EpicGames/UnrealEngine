// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_SkeletalControlBase.h"
#include "AnimNode_WheelController.h"
#include "AnimGraphNode_WheelController.generated.h"

class UEdGraph;

UCLASS(MinimalAPI, meta=(Keywords = "Modify Wheel Vehicle"))
class UAnimGraphNode_WheelController : public UAnimGraphNode_SkeletalControlBase
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_WheelController Node;

public:
	// UEdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	// validate if this is within VehicleAnimInstance
	virtual void ValidateAnimNodePostCompile(class FCompilerResultsLog& MessageLog, class UAnimBlueprintGeneratedClass* CompiledClass, int32 CompiledNodeIndex) override;
	virtual bool IsCompatibleWithGraph(const UEdGraph* TargetGraph) const override;
	// End of UEdGraphNode interface

protected:
	// UAnimGraphNode_SkeletalControlBase interface
	virtual FText GetControllerDescription() const override;
	virtual const FAnimNode_SkeletalControlBase* GetNode() const override { return &Node; }
	// End of UAnimGraphNode_SkeletalControlBase interface
};
