// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Toolkits/IToolkitHost.h"
#include "AssetTypeActions_Base.h"

class FAssetTypeActions_ChaosVehicles : public FAssetTypeActions_Base
{
public:
	// IAssetTypeActions Implementation
	virtual FText GetName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_ChaosVehicles", "Chaos Vehicles"); }
	virtual void GetActions(const TArray<UObject*>& InObjects, struct FToolMenuSection& Section) override;
	virtual FColor GetTypeColor() const override { return FColor(255, 192, 128); }
	virtual UClass* GetSupportedClass() const override;
	virtual void OpenAssetEditor(const TArray<UObject*>& InObjects, TSharedPtr<class IToolkitHost> EditWithinLevelEditor = TSharedPtr<IToolkitHost>()) override;
	virtual uint32 GetCategories() override { return EAssetTypeCategories::Physics; }
	virtual UThumbnailInfo* GetThumbnailInfo(UObject* Asset) const override;
};

