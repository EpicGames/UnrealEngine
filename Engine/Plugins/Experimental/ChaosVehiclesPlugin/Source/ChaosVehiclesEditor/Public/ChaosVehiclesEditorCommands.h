// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Engine/World.h"

class UStaticMesh;
class USkeletalMesh;

/**
* The public interface to this module
*/
class CHAOSVEHICLESEDITOR_API FChaosVehiclesCommands
{
};