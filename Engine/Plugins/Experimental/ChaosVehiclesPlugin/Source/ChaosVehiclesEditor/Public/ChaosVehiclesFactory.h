// Copyright Epic Games, Inc. All Rights Reserved.

/** Factory which allows import of an ChaosVehiclesAsset */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
//#include "ChaosVehicles.h"
//
//#include "ChaosVehiclesFactory.generated.h"
//
//
//UCLASS()
//class CHAOSVEHICLESEDITOR_API UChaosVehiclesFactory : public UFactory
//{
//    GENERATED_UCLASS_BODY()
//
//	//~ Begin UFactory Interface
//	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
//	static UChaosVehicles* StaticFactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn);
//};
//
//
